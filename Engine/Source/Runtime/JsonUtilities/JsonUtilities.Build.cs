// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class JsonUtilities : ModuleRules
{
	public JsonUtilities(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Json",
			}
		);
	}
}
