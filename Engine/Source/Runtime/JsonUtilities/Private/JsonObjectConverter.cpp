// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonObjectConverter.h"
#include "JsonObjectStructInterface.h"
#include "JsonObjectWrapper.h"
#include "Internationalization/Culture.h"
#include "Misc/PackageName.h"
#include "Misc/ScopeExit.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "UObject/StrProperty.h"
#include "UObject/UnrealType.h"
#include "UObject/EnumProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/PropertyOptional.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/Package.h"
#include "UObject/VerseStringProperty.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "VerseVM/VVMNames.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/PropertyBag.h"

#define LOCTEXT_NAMESPACE "JsonObjectConverter"

FString FJsonObjectConverter::StandardizeCase(const FString &StringIn)
{
	FString FixedString = StringIn;
	StandardizeCaseInPlace(FixedString);
	return FixedString;
}

void FJsonObjectConverter::StandardizeCaseInPlace(FString& String)
{
	if (String.IsEmpty())
	{
		return;
	}

	// this probably won't work for all cases, consider downcasing the string fully
	String[0] = FChar::ToLower(String[0]); // our JSON classes/variable start lower case
	String.ReplaceInline(TEXT("ID"), TEXT("Id"), ESearchCase::CaseSensitive); // Id is standard instead of ID, some of our fnames use ID
}

namespace
{
	const FString ObjectClassNameKey = "_ClassName";

	const FName NAME_DateTime("DateTime");
	const FName NAME_Guid("Guid");

	TSharedPtr<FJsonValue> FPropertyToJsonValueWithContainer(FProperty* Property, const void* Value, const UObject* Container, TSet<const UObject*>* ExportedObjects, int64 CheckFlags, int64 SkipFlags, const FJsonObjectConverter::CustomExportCallback* ExportCb, FProperty* OuterProperty, EJsonObjectConversionFlags ConversionFlags);
	bool UStructToJsonAttributesWithContainer(const UStruct* StructDefinition, const void* Struct, const UObject* Container, TSet<const UObject*>* ExportedObjects, TSharedRef<FJsonObject> OutJsonObject, int64 CheckFlags, int64 SkipFlags, const FJsonObjectConverter::CustomExportCallback* ExportCb, EJsonObjectConversionFlags ConversionFlags);
	
	bool ShouldExportObjectPropertyByValue(const FObjectProperty* Property, const UObject* Value, const UObject* Container, const TSet<const UObject*>* ExportedObjects, const FProperty* OuterProperty)
	{
		// Nothing to export if NULL.
		if (!Value)
		{
			return false;
		}

		// Check cycle guards BEFORE the instanced flag

		// Check if it's a reference to the container (self) to guard against cycles.
		if (Value == Container)
		{
			return false;
		}

		// Check if we've already exported this value.
		if (ExportedObjects
			&& ExportedObjects->Contains(Value))
		{
			return false;
		}

		// Check the instanced flag for backwards compatibility - always export by value in this case.
		if (Property->HasAnyPropertyFlags(CPF_PersistentInstance)
			|| (OuterProperty && OuterProperty->HasAnyPropertyFlags(CPF_PersistentInstance)))
		{
			return true;
		}

		// Export by value if it is scoped within the current container context (if set).
		if (Container
			&& Value->IsInOuter(Container))
		{
			return true;
		}

		return false;
	}

/** Convert property to JSON, assuming either the property is not an array or the value is an individual array element */
TSharedPtr<FJsonValue> ConvertScalarFPropertyToJsonValueWithContainer(FProperty* Property, const void* Value, const UObject* Container, TSet<const UObject*>* ExportedObjects, int64 CheckFlags, int64 SkipFlags, const FJsonObjectConverter::CustomExportCallback* ExportCb, FProperty* OuterProperty, EJsonObjectConversionFlags ConversionFlags)
{
	// See if there's a custom export callback first, so it can override default behavior
	if (ExportCb && ExportCb->IsBound())
	{
		TSharedPtr<FJsonValue> CustomValue = ExportCb->Execute(Property, Value);
		if (CustomValue.IsValid())
		{
			return CustomValue;
		}
		// fall through to default cases
	}

	if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		// export enums as strings
		UEnum* EnumDef = EnumProperty->GetEnum();
		FString StringValue = EnumDef->GetAuthoredNameStringByValue(EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(Value));
		return MakeShared<FJsonValueString>(StringValue);
	}
	else if (FNumericProperty *NumericProperty = CastField<FNumericProperty>(Property))
	{
		// see if it's an enum
		UEnum* EnumDef = NumericProperty->GetIntPropertyEnum();
		if (EnumDef != NULL)
		{
			// export enums as strings
			FString StringValue = EnumDef->GetAuthoredNameStringByValue(NumericProperty->GetSignedIntPropertyValue(Value));
			return MakeShared<FJsonValueString>(StringValue);
		}

		// We want to export numbers as numbers
		if (NumericProperty->IsFloatingPoint())
		{
			return MakeShared<FJsonValueNumber>(NumericProperty->GetFloatingPointPropertyValue(Value));
		}
		else if (NumericProperty->IsInteger())
		{
			return MakeShared<FJsonValueNumber>(NumericProperty->GetSignedIntPropertyValue(Value));
		}

		// fall through to default
	}
	else if (FBoolProperty *BoolProperty = CastField<FBoolProperty>(Property))
	{
		// Export bools as bools
		return MakeShared<FJsonValueBoolean>(BoolProperty->GetPropertyValue(Value));
	}
	else if (FStrProperty *StringProperty = CastField<FStrProperty>(Property))
	{
		return MakeShared<FJsonValueString>(StringProperty->GetPropertyValue(Value));
	}
	else if (FTextProperty *TextProperty = CastField<FTextProperty>(Property))
	{
		if (EnumHasAnyFlags(ConversionFlags, EJsonObjectConversionFlags::WriteTextAsComplexString))
		{
			FString TextValueString;
			FTextStringHelper::WriteToBuffer(TextValueString, TextProperty->GetPropertyValue(Value));

			return MakeShared<FJsonValueString>(TextValueString);
		}

		return MakeShared<FJsonValueString>(TextProperty->GetPropertyValue(Value).ToString());
	}
	else if (FArrayProperty *ArrayProperty = CastField<FArrayProperty>(Property))
	{
		TArray< TSharedPtr<FJsonValue> > Out;
		FScriptArrayHelper Helper(ArrayProperty, Value);
		for (int32 i=0, n=Helper.Num(); i<n; ++i)
		{
			TSharedPtr<FJsonValue> Elem = FPropertyToJsonValueWithContainer(ArrayProperty->Inner, Helper.GetRawPtr(i), Container, ExportedObjects, CheckFlags & ( ~CPF_ParmFlags ), SkipFlags, ExportCb, ArrayProperty, ConversionFlags);
			if ( Elem.IsValid() )
			{
				// add to the array
				Out.Push(Elem);
			}
		}
		return MakeShared<FJsonValueArray>(Out);
	}
	else if ( FSetProperty* SetProperty = CastField<FSetProperty>(Property) )
	{
		TArray< TSharedPtr<FJsonValue> > Out;
		FScriptSetHelper Helper(SetProperty, Value);
		for (FScriptSetHelper::FIterator It(Helper); It; ++It)
		{
			TSharedPtr<FJsonValue> Elem = FPropertyToJsonValueWithContainer(SetProperty->ElementProp, Helper.GetElementPtr(It), Container, ExportedObjects, CheckFlags & (~CPF_ParmFlags), SkipFlags, ExportCb, SetProperty, ConversionFlags);
			if (Elem.IsValid())
			{
				// add to the array
				Out.Push(Elem);
			}
		}
		return MakeShared<FJsonValueArray>(Out);
	}
	else if ( FMapProperty* MapProperty = CastField<FMapProperty>(Property) )
	{
		TSharedRef<FJsonObject> Out = MakeShared<FJsonObject>();

		FScriptMapHelper Helper(MapProperty, Value);
		for (FScriptMapHelper::FIterator It(Helper); It; ++It)
		{
			TSharedPtr<FJsonValue> KeyElement = FPropertyToJsonValueWithContainer(MapProperty->KeyProp, Helper.GetKeyPtr(It), Container, ExportedObjects, CheckFlags & (~CPF_ParmFlags), SkipFlags, ExportCb, MapProperty, ConversionFlags);
			TSharedPtr<FJsonValue> ValueElement = FPropertyToJsonValueWithContainer(MapProperty->ValueProp, Helper.GetValuePtr(It), Container, ExportedObjects, CheckFlags & (~CPF_ParmFlags), SkipFlags, ExportCb, MapProperty, ConversionFlags);
			if (KeyElement.IsValid() && ValueElement.IsValid())
			{
				FString KeyString;
				if (!KeyElement->TryGetString(KeyString))
				{
					MapProperty->KeyProp->ExportTextItem_Direct(KeyString, Helper.GetKeyPtr(It), nullptr, nullptr, 0);
					if (KeyString.IsEmpty())
					{
						UE_LOGF(LogJson, Error, "Unable to convert key to string for property %ls.", *MapProperty->GetAuthoredName())
						KeyString = FString::Printf(TEXT("Unparsed Key %d"), It.GetLogicalIndex());
					}
				}

				// Coerce camelCase map keys for Enum/FName properties
				if (CastField<FEnumProperty>(MapProperty->KeyProp) ||
					CastField<FNameProperty>(MapProperty->KeyProp))
				{
					if (!EnumHasAnyFlags(ConversionFlags, EJsonObjectConversionFlags::SkipStandardizeCase))
					{
						FJsonObjectConverter::StandardizeCaseInPlace(KeyString);
					}
				}
				Out->SetField(KeyString, ValueElement);
			}
		}

		return MakeShared<FJsonValueObject>(Out);
	}
	else if (FStructProperty *StructProperty = CastField<FStructProperty>(Property))
	{
		// Intentionally exclude the JSON Object wrapper, which specifically needs to export JSON in an object representation instead of a string
		if (StructProperty->Struct != FJsonObjectWrapper::StaticStruct())
		{
			if (const IJsonObjectStructConverter* ConverterInterface = UE::Json::Private::GetStructConverterInterface(StructProperty->Struct))
			{
				TSharedPtr<FJsonObject> OutFromConverter = MakeShared<FJsonObject>();
				switch (ConverterInterface->ConvertToJson(Value, OutFromConverter))
				{
				case EJsonObjectConvertResult::UseDefaultConverter:
				{
					// Continue to default converter.
					break;
				}
				case EJsonObjectConvertResult::FailAndAbort:
				{
					// A failed ConvertToJson(...) gets treated as a null TSharedPtr, which can be seen as an error and may also abort the entire conversion.
					// For example, UStructToJsonAttributesWithContainer will check JsonValue.IsValid() and treat invalid values as errors. 
					return TSharedPtr<FJsonValue>();
				}
				case EJsonObjectConvertResult::IgnoreAndContinue:
				{
					// An ignored ConvertToJson(...) gets treated as an empty object (i.e. '{}'), but will not fail the conversion tree. 
					return MakeShared<FJsonValueObject>(MakeShared<FJsonObject>());
				}
				case EJsonObjectConvertResult::Converted:
				{
					return MakeShared<FJsonValueObject>(OutFromConverter);
				}
				default:
				{
					unimplemented();
					break;
				}
				}
			}

			// FInstancedStruct: serialize the inner struct's fields as a flat object with _structType for round-trip.
			if (StructProperty->Struct == FInstancedStruct::StaticStruct())
			{
				const FInstancedStruct* InstancedStruct = static_cast<const FInstancedStruct*>(Value);
				if (!InstancedStruct->IsValid())
				{
					return MakeShared<FJsonValueObject>(MakeShared<FJsonObject>());
				}
				TSharedRef<FJsonObject> Out = MakeShared<FJsonObject>();
				Out->SetStringField(TEXT("_structType"), InstancedStruct->GetScriptStruct()->GetPathName());
				if (UStructToJsonAttributesWithContainer(InstancedStruct->GetScriptStruct(), InstancedStruct->GetMemory(), Container, ExportedObjects, Out, CheckFlags & (~CPF_ParmFlags), SkipFlags, ExportCb, ConversionFlags))
				{
					return MakeShared<FJsonValueObject>(Out);
				}
				return TSharedPtr<FJsonValue>();
			}

			// FInstancedPropertyBag: serialize the contained property values as a flat object.
			// Import assumes the bag is already initialized with the correct schema.
			if (StructProperty->Struct == FInstancedPropertyBag::StaticStruct())
			{
				const FInstancedPropertyBag* Bag = static_cast<const FInstancedPropertyBag*>(Value);
				if (!Bag->IsValid())
				{
					return MakeShared<FJsonValueObject>(MakeShared<FJsonObject>());
				}
				const UPropertyBag* BagStruct = Bag->GetPropertyBagStruct();
				if (!BagStruct)
				{
					return MakeShared<FJsonValueObject>(MakeShared<FJsonObject>());
				}
				TSharedRef<FJsonObject> Out = MakeShared<FJsonObject>();
				if (UStructToJsonAttributesWithContainer(BagStruct, Bag->GetValue().GetMemory(), Container, ExportedObjects, Out, CheckFlags & (~CPF_ParmFlags), SkipFlags, ExportCb, ConversionFlags))
				{
					return MakeShared<FJsonValueObject>(Out);
				}
				return TSharedPtr<FJsonValue>();
			}

			// Default converter behavior is to use ExportTextItem if the struct has that function.
			UScriptStruct::ICppStructOps* TheCppStructOps = StructProperty->Struct->GetCppStructOps();
			if (TheCppStructOps && TheCppStructOps->HasExportTextItem())
			{
				FString OutValueStr;
				TheCppStructOps->ExportTextItem(OutValueStr, Value, nullptr, nullptr, PPF_None, nullptr);
				return MakeShared<FJsonValueString>(OutValueStr);
			}
		}

		TSharedRef<FJsonObject> Out = MakeShared<FJsonObject>();
		if (UStructToJsonAttributesWithContainer(StructProperty->Struct, Value, Container, ExportedObjects, Out, CheckFlags & (~CPF_ParmFlags), SkipFlags, ExportCb, ConversionFlags))
		{
			return MakeShared<FJsonValueObject>(Out);
		}
	}
	else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
	{
		// Instanced properties should be copied by value, while normal UObject* properties should output as asset references
		UObject* Object = ObjectProperty->GetObjectPropertyValue(Value);
		if (ShouldExportObjectPropertyByValue(ObjectProperty, Object, Container, ExportedObjects, OuterProperty))
		{
			TSharedRef<FJsonObject> Out = MakeShared<FJsonObject>();
			if (!EnumHasAnyFlags(ConversionFlags, EJsonObjectConversionFlags::SuppressClassNameForPersistentObject))
			{
				Out->SetStringField(ObjectClassNameKey, Object->GetClass()->GetPathName());
			}

			// Track it to ensure that we only export this object by value once; other instances of this value should export as the object's path (i.e. by reference)
			if (ExportedObjects)
			{
				ExportedObjects->Emplace(Object);
			}

			// Use the subobject as the container context for this conversion so that we only create inner JsonObject values for instanced subobjects contained within.
			// Also note we don't clear the ExportedObjects set here to ensure the subobject does not convert references we've already exported by value on an ancestor.
			if (UStructToJsonAttributesWithContainer(Object->GetClass(), Object, Object, ExportedObjects, Out, CheckFlags, SkipFlags, ExportCb, ConversionFlags))
			{
				TSharedRef<FJsonValueObject> JsonObject = MakeShared<FJsonValueObject>(Out);
				JsonObject->Type = EJson::Object;
				return JsonObject;
			}
		}
		else
		{
			FString StringValue;
			Property->ExportTextItem_Direct(StringValue, Value, nullptr, nullptr, PPF_None);
			return MakeShared<FJsonValueString>(StringValue);
		}
	}
	else if (FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(Property))
	{
		if (const void* InnerValue = OptionalProperty->GetValuePointerForReadIfSet(Value))
		{
			return ConvertScalarFPropertyToJsonValueWithContainer(OptionalProperty->GetValueProperty(), InnerValue, Container, ExportedObjects, CheckFlags, SkipFlags, ExportCb, OptionalProperty, ConversionFlags);
		}
		else
		{
			check(!OptionalProperty->IsSet(Value));
			// Note: Unset optional properties are skipped by UStructToJsonAttributesWithContainer, however custom export callbacks or explicit calls to 
			// UPropertyToJsonValue may still arrive here.
			return MakeShared<FJsonValueNull>();
		}
	}
	else if (FVerseStringProperty* VerseStringProperty = CastField<FVerseStringProperty>(Property))
	{
		const ::Verse::FNativeString* VerseStringValue = VerseStringProperty->GetPropertyValuePtr(Value);
		return MakeShared<FJsonValueString>(FString(*VerseStringValue));
	}
	else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
	{
		return MakeShared<FJsonValueNumber>(ByteProperty->GetPropertyValue(Value));
	}
	else
	{
		// Default to export as string for everything else
		FString StringValue;
		Property->ExportTextItem_Direct(StringValue, Value, NULL, NULL, PPF_None);
		return MakeShared<FJsonValueString>(StringValue);
	}

	// invalid
	return TSharedPtr<FJsonValue>();
}

TSharedPtr<FJsonValue> FPropertyToJsonValueWithContainer(FProperty* Property, const void* Value, const UObject* Container, TSet<const UObject*>* ExportedObjects, int64 CheckFlags, int64 SkipFlags, const FJsonObjectConverter::CustomExportCallback* ExportCb, FProperty* OuterProperty, EJsonObjectConversionFlags ConversionFlags)
{
	if (Property->ArrayDim == 1)
	{
		return ConvertScalarFPropertyToJsonValueWithContainer(Property, Value, Container, ExportedObjects, CheckFlags, SkipFlags, ExportCb, OuterProperty, ConversionFlags);
	}

	TArray< TSharedPtr<FJsonValue> > Array;
	for (int Index = 0; Index != Property->ArrayDim; ++Index)
	{
		Array.Add(ConvertScalarFPropertyToJsonValueWithContainer(Property, (char*)Value + Index * Property->GetElementSize(), Container, ExportedObjects, CheckFlags, SkipFlags, ExportCb, OuterProperty, ConversionFlags));
	}
	return MakeShared<FJsonValueArray>(Array);
}

bool UStructToJsonAttributesWithContainer(const UStruct* StructDefinition, const void* Struct, const UObject* Container, TSet<const UObject*>* ExportedObjects, TSharedRef<FJsonObject> OutJsonObject, int64 CheckFlags, int64 SkipFlags, const FJsonObjectConverter::CustomExportCallback* ExportCb, EJsonObjectConversionFlags ConversionFlags)
{
	if (SkipFlags == 0)
	{
		// If we have no specified skip flags, skip deprecated, transient and skip serialization by default when writing
		SkipFlags |= CPF_Deprecated | CPF_Transient;
	}

	if (StructDefinition == FJsonObjectWrapper::StaticStruct())
	{
		// Just copy it into the object
		const FJsonObjectWrapper* ProxyObject = (const FJsonObjectWrapper*)Struct;

		if (ProxyObject->JsonObject.IsValid())
		{
			*OutJsonObject = *ProxyObject->JsonObject;
		}
		return true;
	}

	for (TFieldIterator<FProperty> It(StructDefinition); It; ++It)
	{
		FProperty* Property = *It;

		// Check to see if we should ignore this property
		if (CheckFlags != 0 && !Property->HasAnyPropertyFlags(CheckFlags))
		{
			continue;
		}
		if (Property->HasAnyPropertyFlags(SkipFlags))
		{
			continue;
		}
		
		const void* Value = Property->ContainerPtrToValuePtr<uint8>(Struct);

		// Skip unset optional properties
		if (const FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(Property))
		{
			if (!OptionalProperty->IsSet(Value))
			{
				continue;
			}
		}

		FString VariableName = Property->GetAuthoredName();
		if (!EnumHasAnyFlags(ConversionFlags, EJsonObjectConversionFlags::SkipStandardizeCase))
		{
			FJsonObjectConverter::StandardizeCaseInPlace(VariableName);
		}

		// convert the property to a FJsonValue
		TSharedPtr<FJsonValue> JsonValue = FPropertyToJsonValueWithContainer(Property, Value, Container, ExportedObjects, CheckFlags, SkipFlags, ExportCb, nullptr, ConversionFlags);
		if (!JsonValue.IsValid())
		{
			FFieldClass* PropClass = Property->GetClass();
			UE_LOGF(LogJson, Error, "UStructToJsonObject - Unhandled property type '%ls': %ls", *PropClass->GetName(), *Property->GetPathName());
			return false;
		}

		// set the value on the output object
		OutJsonObject->SetField(VariableName, JsonValue);
	}

	return true;
}
}

TSharedPtr<FJsonValue> FJsonObjectConverter::UPropertyToJsonValue(FProperty* Property, const void* Value, int64 CheckFlags, int64 SkipFlags, const CustomExportCallback* ExportCb, FProperty* OuterProperty, EJsonObjectConversionFlags ConversionFlags)
{
	return FPropertyToJsonValueWithContainer(Property, Value, nullptr, nullptr, CheckFlags, SkipFlags, ExportCb, OuterProperty, ConversionFlags);
}

bool FJsonObjectConverter::UStructToJsonObject(const UStruct* StructDefinition, const void* Struct, TSharedRef<FJsonObject> OutJsonObject, int64 CheckFlags, int64 SkipFlags, const CustomExportCallback* ExportCb, EJsonObjectConversionFlags ConversionFlags)
{
	return UStructToJsonAttributes(StructDefinition, Struct, OutJsonObject, CheckFlags, SkipFlags, ExportCb, ConversionFlags);
}

bool FJsonObjectConverter::UStructToJsonAttributes(const UStruct* StructDefinition, const void* Struct, TSharedRef<FJsonObject> OutJsonObject, int64 CheckFlags, int64 SkipFlags, const CustomExportCallback* ExportCb, EJsonObjectConversionFlags ConversionFlags)
{
	const UObject* ContainerObject = nullptr;
	if (StructDefinition->IsA<UClass>())
	{
		ContainerObject = static_cast<const UObject*>(Struct);
	}

	TSet<const UObject*> ExportedObjects;
	return UStructToJsonAttributesWithContainer(StructDefinition, Struct, ContainerObject, &ExportedObjects, OutJsonObject, CheckFlags, SkipFlags, ExportCb, ConversionFlags);
}

template<class CharType, class PrintPolicy>
bool UStructToJsonObjectStringInternal(const TSharedRef<FJsonObject>& JsonObject, FString& OutJsonString, int32 Indent)
{
	TSharedRef<TJsonWriter<CharType, PrintPolicy> > JsonWriter = TJsonWriterFactory<CharType, PrintPolicy>::Create(&OutJsonString, Indent);
	bool bSuccess = FJsonSerializer::Serialize(JsonObject, JsonWriter);
	JsonWriter->Close();
	return bSuccess;
}

bool FJsonObjectConverter::UStructToJsonObjectString(const UStruct* StructDefinition, const void* Struct, FString& OutJsonString, int64 CheckFlags, int64 SkipFlags, int32 Indent, const CustomExportCallback* ExportCb, bool bPrettyPrint)
{
	TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();
	if (UStructToJsonObject(StructDefinition, Struct, JsonObject, CheckFlags, SkipFlags, ExportCb))
	{
		bool bSuccess = false;
		if (bPrettyPrint)
		{
			bSuccess = UStructToJsonObjectStringInternal<TCHAR, TPrettyJsonPrintPolicy<TCHAR> >(JsonObject, OutJsonString, Indent);
		}
		else
		{
			bSuccess = UStructToJsonObjectStringInternal<TCHAR, TCondensedJsonPrintPolicy<TCHAR> >(JsonObject, OutJsonString, Indent);
		}
		if (bSuccess)
		{
			return true;
		}
		else
		{
			UE_LOGF(LogJson, Warning, "UStructToJsonObjectString - Unable to write out JSON");
		}
	}

	return false;
}

//static
bool FJsonObjectConverter::GetTextFromObject(const TSharedRef<FJsonObject>& Obj, FText& TextOut)
{
	// get the prioritized culture name list
	FCultureRef CurrentCulture = FInternationalization::Get().GetCurrentCulture();
	TArray<FString> CultureList = CurrentCulture->GetPrioritizedParentCultureNames();

	// try to follow the fall back chain that the engine uses
	FString TextString;
	for (const FString& CultureCode : CultureList)
	{
		if (Obj->TryGetStringField(CultureCode, TextString))
		{
			TextOut = FText::FromString(TextString);
			return true;
		}
	}

	// try again but only search on the locale region (in the localized data). This is a common omission (i.e. en-US source text should be used if no en is defined)
	for (const FString& LocaleToMatch : CultureList)
	{
		int32 SeparatorPos;
		// only consider base language entries in culture chain (i.e. "en")
		if (!LocaleToMatch.FindChar('-', SeparatorPos))
		{
			for (const auto& Pair : Obj->Values)
			{
				// only consider coupled entries now (base ones would have been matched on first path) (i.e. "en-US")
				const FStringView KeyView(Pair.Key);
				if (KeyView.FindChar('-', SeparatorPos))
				{
					if (KeyView.StartsWith(LocaleToMatch))
					{
						TextOut = FText::FromString(Pair.Value->AsString());
						return true;
					}
				}
			}
		}
	}

	// no luck, is this possibly an unrelated JSON object?
	return false;
}


namespace
{
	bool JsonValueToFPropertyWithContainer(const TSharedPtr<FJsonValue>& JsonValue, FProperty* Property, void* OutValue, const UStruct* ContainerStruct, void* Container, int64 CheckFlags, int64 SkipFlags, const bool bStrictMode, FText* OutFailReason, const FJsonObjectConverter::CustomImportCallback* ImportCb);
	bool JsonAttributesToUStructWithContainer(const TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& JsonAttributes, const UStruct* StructDefinition, void* OutStruct, const UStruct* ContainerStruct, void* Container, int64 CheckFlags, int64 SkipFlags, const bool bStrictMode, FText* OutFailReason, const FJsonObjectConverter::CustomImportCallback* ImportCb);

	/** Convert JSON to property, assuming either the property is not an array or the value is an individual array element */
	bool ConvertScalarJsonValueToFPropertyWithContainer(const TSharedPtr<FJsonValue>& JsonValue, FProperty* Property, void* OutValue, const UStruct* ContainerStruct, void* Container, int64 CheckFlags, int64 SkipFlags, const bool bStrictMode, FText* OutFailReason, const FJsonObjectConverter::CustomImportCallback* ImportCb)
	{
		if (ImportCb && ImportCb->IsBound())
		{
			if (ImportCb->Execute(JsonValue, Property, OutValue))
			{
				return true;
			}
			// fall through to default cases
		}

		if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			if (JsonValue->Type == EJson::String)
			{
				// see if we were passed a string for the enum
				const UEnum* Enum = EnumProperty->GetEnum();
				check(Enum);
				FString StrValue = JsonValue->AsString();
				int64 IntValue = Enum->GetValueByName(FName(*StrValue), EGetByNameFlags::CheckAuthoredName);
				if (IntValue == INDEX_NONE)
				{
					UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import enum %ls from string value %ls for property %ls", *Enum->CppType, *StrValue, *Property->GetAuthoredName());
					if (OutFailReason)
					{
						*OutFailReason = FText::Format(LOCTEXT("FailImportEnumFromString", "Unable to import enum {0} from string value {1} for property {2}"), FText::FromString(Enum->CppType), FText::FromString(StrValue), FText::FromString(Property->GetAuthoredName()));
					}
					return false;
				}
				EnumProperty->GetUnderlyingProperty()->SetIntPropertyValue(OutValue, IntValue);
			}
			else
			{
				// AsNumber will log an error for completely inappropriate types (then give us a default)
				EnumProperty->GetUnderlyingProperty()->SetIntPropertyValue(OutValue, (int64)JsonValue->AsNumber());
			}
		}
		else if (FNumericProperty *NumericProperty = CastField<FNumericProperty>(Property))
		{
			if (NumericProperty->IsEnum() && JsonValue->Type == EJson::String)
			{
				// see if we were passed a string for the enum
				const UEnum* Enum = NumericProperty->GetIntPropertyEnum();
				check(Enum); // should be assured by IsEnum()
				FString StrValue = JsonValue->AsString();
				int64 IntValue = Enum->GetValueByName(FName(*StrValue), EGetByNameFlags::CheckAuthoredName);
				if (IntValue == INDEX_NONE)
				{
					UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import enum %ls from numeric value %ls for property %ls", *Enum->CppType, *StrValue, *Property->GetAuthoredName());
					if (OutFailReason)
					{
						*OutFailReason = FText::Format(LOCTEXT("FailImportEnumFromNumeric", "Unable to import enum {0} from numeric value {1} for property {2}"), FText::FromString(Enum->CppType), FText::FromString(StrValue), FText::FromString(Property->GetAuthoredName()));
					}
					return false;
				}
				NumericProperty->SetIntPropertyValue(OutValue, IntValue);
			}
			else if (NumericProperty->IsFloatingPoint())
			{
				// AsNumber will log an error for completely inappropriate types (then give us a default)
				NumericProperty->SetFloatingPointPropertyValue(OutValue, JsonValue->AsNumber());
			}
			else if (NumericProperty->IsInteger())
			{
				if (JsonValue->Type == EJson::String)
				{
					// parse string -> int64 ourselves so we don't lose any precision going through AsNumber (aka double)
					NumericProperty->SetIntPropertyValue(OutValue, FCString::Atoi64(*JsonValue->AsString()));
				}
				else
				{
					// AsNumber will log an error for completely inappropriate types (then give us a default)
					NumericProperty->SetIntPropertyValue(OutValue, (int64)JsonValue->AsNumber());
				}
			}
			else
			{
				UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import json value into %ls numeric property %ls", *Property->GetClass()->GetName(), *Property->GetAuthoredName());
				if (OutFailReason)
				{
					*OutFailReason = FText::Format(LOCTEXT("FailImportNumericProperty", "Unable to import json value into {0} numeric property {1}"), FText::FromString(Property->GetClass()->GetName()), FText::FromString(Property->GetAuthoredName()));
				}
				return false;
			}
		}
		else if (FBoolProperty *BoolProperty = CastField<FBoolProperty>(Property))
		{
			// AsBool will log an error for completely inappropriate types (then give us a default)
			BoolProperty->SetPropertyValue(OutValue, JsonValue->AsBool());
		}
		else if (FStrProperty *StringProperty = CastField<FStrProperty>(Property))
		{
			// AsString will log an error for completely inappropriate types (then give us a default)
			StringProperty->SetPropertyValue(OutValue, JsonValue->AsString());
		}
		else if (FArrayProperty *ArrayProperty = CastField<FArrayProperty>(Property))
		{
			if (JsonValue->Type == EJson::Array)
			{
				TArray< TSharedPtr<FJsonValue> > ArrayValue = JsonValue->AsArray();
				int32 ArrLen = ArrayValue.Num();

				// make the output array size match
				FScriptArrayHelper Helper(ArrayProperty, OutValue);
				Helper.Resize(ArrLen);

				// set the property values
				for (int32 i = 0; i < ArrLen; ++i)
				{
					const TSharedPtr<FJsonValue>& ArrayValueItem = ArrayValue[i];
					if (ArrayValueItem.IsValid() && !ArrayValueItem->IsNull())
					{
						if (!JsonValueToFPropertyWithContainer(ArrayValueItem, ArrayProperty->Inner, Helper.GetRawPtr(i), ContainerStruct, Container, CheckFlags & (~CPF_ParmFlags), SkipFlags, bStrictMode, OutFailReason, ImportCb))
						{
							UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import Array element %d for property %ls", i, *Property->GetAuthoredName());
							if (OutFailReason)
							{
								*OutFailReason = FText::Format(LOCTEXT("FailImportArrayElement", "Unable to import Array element {0} for property {1}\n{2}"), FText::AsNumber(i), FText::FromString(Property->GetAuthoredName()), *OutFailReason);
							}
							return false;
						}
					}
				}
			}
			else
			{
				UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import non-array JSON value into Array property %ls", *Property->GetAuthoredName());
				if (OutFailReason)
				{
					*OutFailReason = FText::Format(LOCTEXT("FailImportArray", "Unable to import non-array JSON value into Array property {0}"), FText::FromString(Property->GetAuthoredName()));
				}
				return false;
			}
		}
		else if (FMapProperty* MapProperty = CastField<FMapProperty>(Property))
		{
			if (JsonValue->Type == EJson::Object)
			{
				TSharedPtr<FJsonObject> ObjectValue = JsonValue->AsObject();

				FScriptMapHelper Helper(MapProperty, OutValue);

				check(ObjectValue);

				int32 MapSize = ObjectValue->Values.Num();
				Helper.EmptyValues(MapSize);

				ON_SCOPE_EXIT
				{
					Helper.Rehash();
				};

				// set the property values
				for (const auto& Entry : ObjectValue->Values)
				{
					if (Entry.Value.IsValid() && !Entry.Value->IsNull())
					{
						int32 NewIndex = Helper.AddDefaultValue_Invalid_NeedsRehash();

						TSharedPtr<FJsonValueString> TempKeyValue = MakeShared<FJsonValueString>(FString(Entry.Key));

						if (!JsonValueToFPropertyWithContainer(TempKeyValue, MapProperty->KeyProp, Helper.GetKeyPtr(NewIndex), ContainerStruct, Container, CheckFlags & (~CPF_ParmFlags), SkipFlags, bStrictMode, OutFailReason, ImportCb))
						{
							UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import Map element %ls key for property %ls", *Entry.Key, *Property->GetAuthoredName());
							if (OutFailReason)
							{
								*OutFailReason = FText::Format(LOCTEXT("FailImportMapElementKey", "Unable to import Map element {0} key for property {1}\n{2}"), FText::FromString(FString(Entry.Key)), FText::FromString(Property->GetAuthoredName()), *OutFailReason);
							}
							return false;
						}

						if (!JsonValueToFPropertyWithContainer(Entry.Value, MapProperty->ValueProp, Helper.GetValuePtr(NewIndex), ContainerStruct, Container, CheckFlags & (~CPF_ParmFlags), SkipFlags, bStrictMode, OutFailReason, ImportCb))
						{
							UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import Map element %ls value for property %ls", *Entry.Key, *Property->GetAuthoredName());
							if (OutFailReason)
							{
								*OutFailReason = FText::Format(LOCTEXT("FailImportMapElementValue", "Unable to import Map element {0} value for property {1}\n{2}"), FText::FromString(FString(Entry.Key)), FText::FromString(Property->GetAuthoredName()), *OutFailReason);
							}
							return false;
						}
					}
				}
			}
			else
			{
				UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import non-object JSON value into Map property %ls", *Property->GetAuthoredName());
				if (OutFailReason)
				{
					*OutFailReason = FText::Format(LOCTEXT("FailImportMap", "Unable to import non-object JSON value into Map property {0}"), FText::FromString(Property->GetAuthoredName()));
				}
				return false;
			}
		}
		else if (FSetProperty* SetProperty = CastField<FSetProperty>(Property))
		{
			if (JsonValue->Type == EJson::Array)
			{
				TArray< TSharedPtr<FJsonValue> > ArrayValue = JsonValue->AsArray();
				int32 ArrLen = ArrayValue.Num();

				FScriptSetHelper Helper(SetProperty, OutValue);
				Helper.EmptyElements(ArrLen);

				ON_SCOPE_EXIT
				{
					Helper.Rehash();
				};

				// set the property values
				for (int32 i = 0; i < ArrLen; ++i)
				{
					const TSharedPtr<FJsonValue>& ArrayValueItem = ArrayValue[i];
					if (ArrayValueItem.IsValid() && !ArrayValueItem->IsNull())
					{
						int32 NewIndex = Helper.AddDefaultValue_Invalid_NeedsRehash();
						if (!JsonValueToFPropertyWithContainer(ArrayValueItem, SetProperty->ElementProp, Helper.GetElementPtr(NewIndex), ContainerStruct, Container, CheckFlags & (~CPF_ParmFlags), SkipFlags, bStrictMode, OutFailReason, ImportCb))
						{
							UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import Set element %d for property %ls", i, *Property->GetAuthoredName());
							if (OutFailReason)
							{
								*OutFailReason = FText::Format(LOCTEXT("FailImportSetElement", "Unable to import Set element {0} for property {1}\n{2}"), FText::AsNumber(i), FText::FromString(Property->GetAuthoredName()), *OutFailReason);
							}
							return false;
						}
					}
				}
			}
			else
			{
				UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import non-array JSON value into Set property %ls", *Property->GetAuthoredName());
				if (OutFailReason)
				{
					*OutFailReason = FText::Format(LOCTEXT("FailImportSet", "Unable to import non-array JSON value into Set property {0}"), FText::FromString(Property->GetAuthoredName()));
				}
				return false;
			}
		}
		else if (FTextProperty* TextProperty = CastField<FTextProperty>(Property))
		{
			if (JsonValue->Type == EJson::String)
			{
				FString StringValue = JsonValue->AsString();
				FText TextValue;
				if (!FTextStringHelper::ReadFromBuffer(*StringValue, TextValue))
				{
					TextValue = FText::FromString(StringValue);
				}

				// assume this string is already localized, so import as invariant
				TextProperty->SetPropertyValue(OutValue, TextValue);
			}
			else if (JsonValue->Type == EJson::Object)
			{
				TSharedPtr<FJsonObject> Obj = JsonValue->AsObject();
				check(Obj.IsValid()); // should not fail if Type == EJson::Object
	
				// import the subvalue as a culture invariant string
				FText Text;
				if (!FJsonObjectConverter::GetTextFromObject(Obj.ToSharedRef(), Text))
				{
					UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import JSON object with invalid keys into Text property %ls", *Property->GetAuthoredName());
					if (OutFailReason)
					{
						*OutFailReason = FText::Format(LOCTEXT("FailImportTextFromObject", "Unable to import JSON object with invalid keys into Text property {0}"), FText::FromString(Property->GetAuthoredName()));
					}
					return false;
				}
				TextProperty->SetPropertyValue(OutValue, Text);
			}
			else
			{
				UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import JSON value that is neither string nor object into Text property %ls", *Property->GetAuthoredName());
				if (OutFailReason)
				{
					*OutFailReason = FText::Format(LOCTEXT("FailImportText", "Unable to import JSON value that is neither string nor object into Text property {0}"), FText::FromString(Property->GetAuthoredName()));
				}
				return false;
			}
		}
		else if (FStructProperty *StructProperty = CastField<FStructProperty>(Property))
		{
			if (JsonValue->Type == EJson::Object)
			{
				TSharedPtr<FJsonObject> Obj = JsonValue->AsObject();
				check(Obj.IsValid()); // should not fail if Type == EJson::Object

				bool bUseDefaultConverter = true;

				if (const IJsonObjectStructConverter* ConverterInterface = UE::Json::Private::GetStructConverterInterface(StructProperty->Struct))
				{
					switch (ConverterInterface->ConvertFromJson(OutValue, Obj))
					{
					case EJsonObjectConvertResult::UseDefaultConverter:
					{
						bUseDefaultConverter = true;
						break;
					}
					case EJsonObjectConvertResult::FailAndAbort:
					{
						UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to convert JSON object into %ls property %ls while use a IJsonObjectStructConverter", *StructProperty->Struct->GetAuthoredName(), *Property->GetAuthoredName());
						if (OutFailReason)
						{
							*OutFailReason = FText::Format(LOCTEXT("FailConvertStructFromObject", "Unable to convert JSON object into {0} property {1}\n{2} while using a IJsonObjectStructConverter"), FText::FromString(StructProperty->Struct->GetAuthoredName()), FText::FromString(Property->GetAuthoredName()), *OutFailReason);
						}
						return false;
					}
					case EJsonObjectConvertResult::IgnoreAndContinue:
					case EJsonObjectConvertResult::Converted:
					{
						// IgnoreAndContinue and Converted are treated the same here
						bUseDefaultConverter = false;
						break;
					}
					default:
					{
						unimplemented();
						break;
					}
					}
				}

				if (bUseDefaultConverter)
				{
					// This is the struct that will be used for output.
					const UScriptStruct* ScriptStruct = nullptr;

					// FInstancedStruct: applies the JSON fields to the struct type stored in the instance.
					// If not yet initialized, reads _structType from JSON to create the correct type.
					if (StructProperty->Struct == FInstancedStruct::StaticStruct())
					{
						FInstancedStruct* InstancedStruct = static_cast<FInstancedStruct*>(OutValue);
						if (!InstancedStruct->IsValid())
						{
							FString StructTypePath;
							if (Obj->TryGetStringField(TEXT("_structType"), StructTypePath))
							{
								UScriptStruct* FoundStruct = FindObject<UScriptStruct>(nullptr, *StructTypePath);
								if (FoundStruct == nullptr)
								{
									UE_LOGF(LogJson, Error, "JsonValueToUProperty - _structType '%ls' could not be resolved for %ls property %ls",
										*StructTypePath, *StructProperty->Struct->GetAuthoredName(), *Property->GetAuthoredName());
									if (OutFailReason)
									{
										*OutFailReason = FText::Format(
											LOCTEXT("FailImportInstancedStructType", "_structType '{0}' could not be resolved for {1} property {2}"),
											FText::FromString(StructTypePath), FText::FromString(StructProperty->Struct->GetAuthoredName()),
											FText::FromString(Property->GetAuthoredName()));
									}
									return false;
								}
								InstancedStruct->InitializeAs(FoundStruct);
							}
						}
						if (InstancedStruct->IsValid())
						{
							ScriptStruct = InstancedStruct->GetScriptStruct();
							OutValue = InstancedStruct->GetMutableMemory();
						}
						else
						{
							// No _structType and not pre-initialized -- empty FInstancedStruct, leave as-is.
							return true;
						}
					}
					// FInstancedPropertyBag: flat property values; bag must already be initialized with the correct schema.
					else if (StructProperty->Struct == FInstancedPropertyBag::StaticStruct())
					{
						FInstancedPropertyBag* Bag = static_cast<FInstancedPropertyBag*>(OutValue);
						if (Bag->IsValid())
						{
							const UPropertyBag* BagStruct = Bag->GetPropertyBagStruct();
							if (BagStruct)
							{
								ScriptStruct = BagStruct;
								OutValue = Bag->GetMutableValue().GetMemory();
							}
						}
					}
					// Traditional UStructs take this path.
					else
					{
						ScriptStruct = StructProperty->Struct;
					}

					if (!ScriptStruct || !JsonAttributesToUStructWithContainer(Obj->Values, ScriptStruct, OutValue, ContainerStruct, Container, CheckFlags & (~CPF_ParmFlags), SkipFlags, bStrictMode, OutFailReason, ImportCb))
					{
						UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import JSON object into %ls property %ls", *StructProperty->Struct->GetAuthoredName(), *Property->GetAuthoredName());
						if (OutFailReason)
						{
							*OutFailReason = FText::Format(LOCTEXT("FailImportStructFromObject", "Unable to import JSON object into {0} property {1}\n{2}"), FText::FromString(StructProperty->Struct->GetAuthoredName()), FText::FromString(Property->GetAuthoredName()), *OutFailReason);
						}
						return false;
					}
				}
			}
			else if (JsonValue->Type == EJson::String && StructProperty->Struct->GetFName() == NAME_LinearColor)
			{
				FLinearColor& ColorOut = *(FLinearColor*)OutValue;
				FString ColorString = JsonValue->AsString();
	
				FColor IntermediateColor;
				IntermediateColor = FColor::FromHex(ColorString);
	
				ColorOut = IntermediateColor;
			}
			else if (JsonValue->Type == EJson::String && StructProperty->Struct->GetFName() == NAME_Color)
			{
				FColor& ColorOut = *(FColor*)OutValue;
				FString ColorString = JsonValue->AsString();

				ColorOut = FColor::FromHex(ColorString);
			}
			else if (JsonValue->Type == EJson::String && StructProperty->Struct->GetFName() == NAME_DateTime)
			{
				FString DateString = JsonValue->AsString();
				FDateTime& DateTimeOut = *(FDateTime*)OutValue;
				if (DateString == TEXT("min"))
				{
					// min representable value for our date struct. Actual date may vary by platform (this is used for sorting)
					DateTimeOut = FDateTime::MinValue();
				}
				else if (DateString == TEXT("max"))
				{
					// max representable value for our date struct. Actual date may vary by platform (this is used for sorting)
					DateTimeOut = FDateTime::MaxValue();
				}
				else if (DateString == TEXT("now"))
				{
					// this value's not really meaningful from JSON serialization (since we don't know timezone) but handle it anyway since we're handling the other keywords
					DateTimeOut = FDateTime::UtcNow();
				}
				else if (FDateTime::ParseIso8601(*DateString, DateTimeOut))
				{
					// ok
				}
				else if (FDateTime::Parse(DateString, DateTimeOut))
				{
					// ok
				}
				else
				{
					UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import JSON string into DateTime property %ls", *Property->GetAuthoredName());
					if (OutFailReason)
					{
						*OutFailReason = FText::Format(LOCTEXT("FailImportDateTimeFromString", "Unable to import JSON string into DateTime property {0}"), FText::FromString(Property->GetAuthoredName()));
					}
					return false;
				}
			}
			else if (JsonValue->Type == EJson::String && StructProperty->Struct->GetFName() == NAME_Guid)
			{
				FString GuidString = JsonValue->AsString();
				FGuid& GuidOut = *(FGuid*)OutValue;
				if (FGuid::Parse(GuidString, GuidOut))
				{
					// ok
				}
				// Try again with regular ImportText if FGuid::Parse fails
				else if (Property->ImportText_Direct(*GuidString, OutValue, nullptr, PPF_None))
				{
					// ok
				}
				else
				{
					UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import JSON string into Guid property %ls", *Property->GetAuthoredName());
					if (OutFailReason)
					{
						*OutFailReason = FText::Format(LOCTEXT("FailImportGuidFromString", "Unable to import JSON string into Guid property {0}"), FText::FromString(Property->GetAuthoredName()));
					}
					return false;
				}
			}
			else if (JsonValue->Type == EJson::String && StructProperty->Struct->GetCppStructOps() && StructProperty->Struct->GetCppStructOps()->HasImportTextItem())
			{
				UScriptStruct::ICppStructOps* TheCppStructOps = StructProperty->Struct->GetCppStructOps();

				FString ImportTextString = JsonValue->AsString();
				const TCHAR* ImportTextPtr = *ImportTextString;
				if (!TheCppStructOps->ImportTextItem(ImportTextPtr, OutValue, PPF_None, nullptr, (FOutputDevice*)GWarn))
				{
					// Fall back to trying the tagged property approach if custom ImportTextItem couldn't get it done
					if (Property->ImportText_Direct(ImportTextPtr, OutValue, nullptr, PPF_None) == nullptr)
					{
						UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import JSON string into %ls property %ls", *StructProperty->Struct->GetAuthoredName(), *Property->GetAuthoredName());
						if (OutFailReason)
						{
							*OutFailReason = FText::Format(LOCTEXT("FailImportStructFromString", "Unable to import JSON string into {0} property {1}"), FText::FromString(StructProperty->Struct->GetAuthoredName()), FText::FromString(Property->GetAuthoredName()));
						}
						return false;
					}
				}
			}
			else if (JsonValue->Type == EJson::String)
			{
				FString ImportTextString = JsonValue->AsString();
				const TCHAR* ImportTextPtr = *ImportTextString;
				if (Property->ImportText_Direct(ImportTextPtr, OutValue, nullptr, PPF_None) == nullptr)
				{
					UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import JSON string into %ls property %ls", *StructProperty->Struct->GetAuthoredName(), *Property->GetAuthoredName());
					if (OutFailReason)
					{
						*OutFailReason = FText::Format(LOCTEXT("FailImportStructFromString", "Unable to import JSON string into {0} property {1}"), FText::FromString(StructProperty->Struct->GetAuthoredName()), FText::FromString(Property->GetAuthoredName()));
					}
					return false;
				}
			}
			else
			{
				UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import JSON value that is neither string nor object into %ls property %ls", *StructProperty->Struct->GetAuthoredName(), *Property->GetAuthoredName());
				if (OutFailReason)
				{
					*OutFailReason = FText::Format(LOCTEXT("FailImportStruct", "Unable to import JSON value that is neither string nor object into {0} property {1}"), FText::FromString(StructProperty->Struct->GetAuthoredName()), FText::FromString(Property->GetAuthoredName()));
				}
				return false;
			}
		}
		else if (FObjectProperty *ObjectProperty = CastField<FObjectProperty>(Property))
		{
			if (JsonValue->Type == EJson::Object)
			{
				UObject* Outer = GetTransientPackage();
				if (ContainerStruct && ContainerStruct->IsChildOf(UObject::StaticClass()))
				{
					Outer = (UObject*)Container;
				}

				TSharedPtr<FJsonObject> Obj = JsonValue->AsObject();
				UClass* PropertyClass = ObjectProperty->PropertyClass;
				
				if (!ObjectProperty->HasAnyPropertyFlags(CPF_InstancedReference | CPF_PersistentInstance | CPF_ContainsInstancedReference) && !PropertyClass->HasAnyClassFlags(CLASS_DefaultToInstanced))
				{
					UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import JSON object into property %ls (property is not instanced)", *Property->GetAuthoredName());
					if (OutFailReason)
					{
						*OutFailReason = FText::Format(LOCTEXT("FailImportObjectNotInstanced", "Unable to import JSON object into property {0} (property is not instanced)"), FText::FromString(Property->GetAuthoredName()));
					}
					return false;
				}

				// If a specific subclass was stored in the JSON, use that instead of the PropertyClass
				FString ClassString;
				if (Obj->TryGetStringField(ObjectClassNameKey, ClassString))
				{
					Obj->RemoveField(ObjectClassNameKey);
					if (!ClassString.IsEmpty())
					{
						UClass* FoundClass = FPackageName::IsShortPackageName(ClassString) ? FindFirstObject<UClass>(*ClassString) : LoadClass<UObject>(nullptr, *ClassString);
						if (FoundClass)
						{
							if (!FoundClass->IsChildOf(PropertyClass) || FoundClass->HasAnyClassFlags(CLASS_Abstract))
							{
								UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import JSON object of type %ls into property %ls of type %ls", *GetNameSafe(FoundClass), *Property->GetAuthoredName(), *GetNameSafe(PropertyClass));
								if (OutFailReason)
								{
									*OutFailReason = FText::Format(
										LOCTEXT("FailImportObjectClass", "Unable to import JSON object of type {0} into property {1} of type {2}"),
										FText::FromString(GetNameSafe(FoundClass)),
										FText::FromString(Property->GetAuthoredName()),
										FText::FromString(GetNameSafe(PropertyClass)));
								}
								return false;
							}

							PropertyClass = FoundClass;
						}
						else
						{
							UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import JSON object for %ls property %ls. Class %ls was not found.", *PropertyClass->GetAuthoredName(), *Property->GetAuthoredName(), *ClassString);
							if (OutFailReason)
							{
								*OutFailReason = FText::Format(LOCTEXT("FailImportObjectClassNotFound", "Unable to import JSON object into {0} property {1}. Class {2} was not found\n{3}"), FText::FromString(PropertyClass->GetAuthoredName()), FText::FromString(Property->GetAuthoredName()), FText::FromString(ClassString), *OutFailReason);
							}
							// Continue execution to keep consistency with original behavior, but log error.
						}
					}
				}

				// Abstract classes should not be instantiated 
				if (PropertyClass->HasAnyClassFlags(CLASS_Abstract))
				{
					UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import JSON object into %ls property %ls. Class is abstract.", *PropertyClass->GetAuthoredName(), *Property->GetAuthoredName());
					if (OutFailReason)
					{
						*OutFailReason = FText::Format(LOCTEXT("FailImportObjectAbstractClass", "Unable to import JSON object into {0} property {1}. Class is abstract\n{2}"), FText::FromString(PropertyClass->GetAuthoredName()), FText::FromString(Property->GetAuthoredName()), *OutFailReason);
					}
					return false;
				}

				UObject* createdObj = StaticAllocateObject(PropertyClass, Outer, NAME_None, EObjectFlags::RF_NoFlags, EInternalObjectFlags::None, false);
				(*PropertyClass->ClassConstructor)(FObjectInitializer(createdObj, PropertyClass->GetDefaultObject(false), EObjectInitializerOptions::None));

				ObjectProperty->SetObjectPropertyValue(OutValue, createdObj);

				check(Obj.IsValid()); // should not fail if Type == EJson::Object
				if (!JsonAttributesToUStructWithContainer(Obj->Values, PropertyClass, createdObj, PropertyClass, createdObj, CheckFlags & (~CPF_ParmFlags), SkipFlags, bStrictMode, OutFailReason, ImportCb))
				{
					UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import JSON object into %ls property %ls", *PropertyClass->GetAuthoredName(), *Property->GetAuthoredName());
					if (OutFailReason)
					{
						*OutFailReason = FText::Format(LOCTEXT("FailImportObjectFromObject", "Unable to import JSON object into {0} property {1}\n{2}"), FText::FromString(PropertyClass->GetAuthoredName()), FText::FromString(Property->GetAuthoredName()), *OutFailReason);
					}
					return false;
				}
			}
			else if (JsonValue->Type == EJson::String)
			{
				// Default to expect a string for everything else
				if (Property->ImportText_Direct(*JsonValue->AsString(), OutValue, nullptr, 0) == nullptr)
				{
					UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import JSON string into %ls property %ls", *ObjectProperty->PropertyClass->GetAuthoredName(), *Property->GetAuthoredName());
					if (OutFailReason)
					{
						*OutFailReason = FText::Format(LOCTEXT("FailImportObjectFromString", "Unable to import JSON string into {0} property {1}"), FText::FromString(*ObjectProperty->PropertyClass->GetAuthoredName()), FText::FromString(Property->GetAuthoredName()));
					}
					return false;
				}
			}
		}
		else if (FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(Property))
		{
			void* OutValueInner = OptionalProperty->MarkSetAndGetInitializedValuePointerToReplace(OutValue);
			return ConvertScalarJsonValueToFPropertyWithContainer(JsonValue, OptionalProperty->GetValueProperty(), OutValueInner, ContainerStruct, Container, CheckFlags, SkipFlags, bStrictMode, OutFailReason, ImportCb);
		}
		else
		{
			// Default to expect a string for everything else
			if (Property->ImportText_Direct(*JsonValue->AsString(), OutValue, nullptr, 0) == nullptr)
			{
				UE_LOGF(LogJson, Error, "JsonValueToUProperty - Unable to import JSON string into property %ls", *Property->GetAuthoredName());
				if (OutFailReason)
				{
					*OutFailReason = FText::Format(LOCTEXT("FailImportFromString", "Unable to import JSON string into property {0}"), FText::FromString(Property->GetAuthoredName()));
				}
				return false;
			}
		}

		return true;
	}


	bool JsonValueToFPropertyWithContainer(const TSharedPtr<FJsonValue>& JsonValue, FProperty* Property, void* OutValue, const UStruct* ContainerStruct, void* Container, int64 CheckFlags, int64 SkipFlags, const bool bStrictMode, FText* OutFailReason, const FJsonObjectConverter::CustomImportCallback* ImportCb)
	{
		if (!JsonValue.IsValid())
		{
			UE_LOGF(LogJson, Error, "JsonValueToUProperty - Invalid JSON value");
			if (OutFailReason)
			{
				*OutFailReason = LOCTEXT("InvalidJsonValue", "Invalid JSON value");
			}
			return false;
		}

		const bool bArrayOrSetProperty = Property->IsA<FArrayProperty>() || Property->IsA<FSetProperty>();
		const bool bJsonArray = JsonValue->Type == EJson::Array;

		if (!bJsonArray)
		{
			if (bArrayOrSetProperty)
			{
				UE_LOGF(LogJson, Error, "JsonValueToUProperty - Expecting JSON array");
				if (OutFailReason)
				{
					*OutFailReason = LOCTEXT("ExpectingJsonArray", "Expecting JSON array");
				}
				return false;
			}

			if (Property->ArrayDim != 1)
			{
				if (bStrictMode)
				{
					UE_LOGF(LogJson, Error, "JsonValueToUProperty - Property %ls is not an array but has %d elements", *Property->GetAuthoredName(), Property->ArrayDim);
					if (OutFailReason)
					{
						*OutFailReason = FText::Format(LOCTEXT("InvalidDimensionOfNonArrayProperty", "Property {0} is not an array but has {1} elements"), FText::FromString(Property->GetAuthoredName()), FText::AsNumber(Property->ArrayDim));
					}
					return false;
				}
				
				UE_LOGF(LogJson, Warning, "Ignoring excess properties when deserializing %ls", *Property->GetAuthoredName());
			}

			return ConvertScalarJsonValueToFPropertyWithContainer(JsonValue, Property, OutValue, ContainerStruct, Container, CheckFlags, SkipFlags, bStrictMode, OutFailReason, ImportCb);
		}

		// In practice, the ArrayDim == 1 check ought to be redundant, since nested arrays of FProperties are not supported
		if (bArrayOrSetProperty && Property->ArrayDim == 1)
		{
			// Read into TArray
			return ConvertScalarJsonValueToFPropertyWithContainer(JsonValue, Property, OutValue, ContainerStruct, Container, CheckFlags, SkipFlags, bStrictMode, OutFailReason, ImportCb);
		}

		// We're deserializing a JSON array
		const auto& ArrayValue = JsonValue->AsArray();

		if (bStrictMode && (Property->ArrayDim != ArrayValue.Num()))
		{
			UE_LOGF(LogJson, Error, "JsonValueToUProperty - JSON array size is incorrect (has %d elements, but needs %d)", ArrayValue.Num(), Property->ArrayDim);
			if (OutFailReason)
			{
				*OutFailReason = FText::Format(LOCTEXT("IncorrectArraySize", "JSON array size is incorrect (has {0} elements, but needs {1})"), FText::AsNumber(ArrayValue.Num()), FText::AsNumber(Property->ArrayDim));
			}
			return false;
		}
		
		if (Property->ArrayDim < ArrayValue.Num())
		{
			UE_LOGF(LogJson, Warning, "Ignoring excess properties when deserializing %ls", *Property->GetAuthoredName());
		}

		// Read into native array
		const int32 ItemsToRead = FMath::Clamp(ArrayValue.Num(), 0, Property->ArrayDim);
		for (int Index = 0; Index != ItemsToRead; ++Index)
		{
			if (!ConvertScalarJsonValueToFPropertyWithContainer(ArrayValue[Index], Property, static_cast<char*>(OutValue) + Index * Property->GetElementSize(), ContainerStruct, Container, CheckFlags, SkipFlags, bStrictMode, OutFailReason, ImportCb))
			{
				return false;
			}
		}
		return true;
	}

	bool JsonAttributesToUStructWithContainer(const TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& JsonAttributes, const UStruct* StructDefinition, void* OutStruct, const UStruct* ContainerStruct, void* Container, int64 CheckFlags, int64 SkipFlags, const bool bStrictMode, FText* OutFailReason, const FJsonObjectConverter::CustomImportCallback* ImportCb)
	{
		if (StructDefinition == FJsonObjectWrapper::StaticStruct())
		{
			// Just copy it into the object
			FJsonObjectWrapper* ProxyObject = (FJsonObjectWrapper*)OutStruct;
			ProxyObject->JsonObject = MakeShared<FJsonObject>();
			ProxyObject->JsonObject->Values = JsonAttributes;
			return true;
		}

		int32 NumUnclaimedProperties = JsonAttributes.Num();
		if (NumUnclaimedProperties <= 0)
		{
			return true;
		}

		// iterate over the struct properties
		for (TFieldIterator<FProperty> PropIt(StructDefinition); PropIt; ++PropIt)
		{
			FProperty* Property = *PropIt;

			// Check to see if we should ignore this property
			if (CheckFlags != 0 && !Property->HasAnyPropertyFlags(CheckFlags))
			{
				continue;
			}
			if (Property->HasAnyPropertyFlags(SkipFlags))
			{
				continue;
			}

			// find a JSON value matching this property name
			FJsonObject::FStringType PropertyName(StructDefinition->GetAuthoredNameForField(Property));
			const TSharedPtr<FJsonValue>* JsonValue = JsonAttributes.Find(PropertyName);
			
			if (!JsonValue)
			{
				if (bStrictMode)
				{
					UE_LOGF(LogJson, Error, "JsonObjectToUStruct - Missing JSON value named %ls", *PropertyName);
					if (OutFailReason)
					{
						*OutFailReason = FText::Format(LOCTEXT("MissingJsonField", "Missing JSON value named {0}"), FText::FromString(*PropertyName));
					}
					return false;
				}
				
				// we allow values to not be found since this mirrors the typical UObject mantra that all the fields are optional when deserializing
				continue;
			}

			if (JsonValue->IsValid() && !(*JsonValue)->IsNull())
			{
				void* Value = Property->ContainerPtrToValuePtr<uint8>(OutStruct);
				if (!JsonValueToFPropertyWithContainer(*JsonValue, Property, Value, ContainerStruct, Container, CheckFlags, SkipFlags, bStrictMode, OutFailReason, ImportCb))
				{
					UE_LOGF(LogJson, Error, "JsonObjectToUStruct - Unable to import JSON value into property %ls", *PropertyName);
					if (OutFailReason)
					{
						*OutFailReason = FText::Format(LOCTEXT("FailImportValueToProperty", "Unable to import JSON value into property {0}\n{1}"), FText::FromString(*PropertyName), *OutFailReason);
					}
					return false;
				}
			}

			if (--NumUnclaimedProperties <= 0)
			{
				// Should we log a warning/error if we still have properties in the JSON data that aren't in the struct definition in strict mode?
				
				// If we found all properties that were in the JsonAttributes map, there is no reason to keep looking for more.
				break;
			}
		}

		return true;
	}
}

bool FJsonObjectConverter::JsonValueToUProperty(const TSharedPtr<FJsonValue>& JsonValue, FProperty* Property, void* OutValue, int64 CheckFlags, int64 SkipFlags, const bool bStrictMode, FText* OutFailReason, const CustomImportCallback* ImportCb)
{
	return JsonValueToFPropertyWithContainer(JsonValue, Property, OutValue, nullptr, nullptr, CheckFlags, SkipFlags, bStrictMode, OutFailReason, ImportCb);
}

bool FJsonObjectConverter::JsonObjectToUStruct(const TSharedRef<FJsonObject>& JsonObject, const UStruct* StructDefinition, void* OutStruct, int64 CheckFlags, int64 SkipFlags, const bool bStrictMode, FText* OutFailReason, const CustomImportCallback* ImportCb)
{
	return JsonAttributesToUStruct(JsonObject->Values, StructDefinition, OutStruct, CheckFlags, SkipFlags, bStrictMode, OutFailReason, ImportCb);
}

bool FJsonObjectConverter::JsonAttributesToUStruct(const TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& JsonAttributes, const UStruct* StructDefinition, void* OutStruct, int64 CheckFlags, int64 SkipFlags, const bool bStrictMode, FText* OutFailReason, const CustomImportCallback* ImportCb)
{
	return JsonAttributesToUStructWithContainer(JsonAttributes, StructDefinition, OutStruct, StructDefinition, OutStruct, CheckFlags, SkipFlags, bStrictMode, OutFailReason, ImportCb);
}

//static 
bool FJsonObjectConverter::GetTextFromField(const FString& FieldName, const TSharedPtr<FJsonValue>& FieldValue, FText& TextOut)
{
	if (FieldValue.IsValid())
	{
		switch (FieldValue->Type)
		{
			case EJson::Number:
			{
				// number
				TextOut = FText::AsNumber(FieldValue->AsNumber());
				return true;
			}
			case EJson::String:
			{
				if (FieldName.StartsWith(TEXT("date-")))
				{
					FDateTime Dte;
					if (FDateTime::ParseIso8601(*FieldValue->AsString(), Dte))
					{
						TextOut = FText::AsDate(Dte);
						return true;
					}
				}
				else if (FieldName.StartsWith(TEXT("datetime-")))
				{
					FDateTime Dte;
					if (FDateTime::ParseIso8601(*FieldValue->AsString(), Dte))
					{
						TextOut = FText::AsDateTime(Dte);
						return true;
					}
				}
				else
				{
				// culture invariant string
					TextOut = FText::FromString(FieldValue->AsString());
					return true;
				}
				break;
			}
			case EJson::Object:
			{
				// localized string
				if (FJsonObjectConverter::GetTextFromObject(FieldValue->AsObject().ToSharedRef(), TextOut))
				{
					return true;
				}

				UE_LOGF(LogJson, Error, "Unable to apply JSON parameter %ls (could not parse object)", *FieldName);
				break;
			}
			default:
			{
				UE_LOGF(LogJson, Error, "Unable to apply JSON parameter %ls (bad type)", *FieldName);
				break;
			}
		}
	}
	return false;
}

FFormatNamedArguments FJsonObjectConverter::ParseTextArgumentsFromJson(const TSharedPtr<const FJsonObject>& JsonObject)
{
	FFormatNamedArguments NamedArgs;
	if (JsonObject.IsValid())
	{
		for (const auto& It : JsonObject->Values)
		{
			FText TextValue;
			FString KeyStr(It.Key);
			if (GetTextFromField(KeyStr, It.Value, TextValue))
			{
				NamedArgs.Emplace(MoveTemp(KeyStr), TextValue);
			}
		}
	}
	return NamedArgs;
}

const FJsonObjectConverter::CustomExportCallback FJsonObjectConverter::ExportCallback_WriteISO8601Dates = 
	FJsonObjectConverter::CustomExportCallback::CreateLambda(
		[](FProperty* Prop, const void* Data) -> TSharedPtr<FJsonValue>
		{
			if (FStructProperty* StructProperty = CastField<FStructProperty>(Prop))
			{
				checkSlow(StructProperty->Struct);
				if (StructProperty->Struct->GetFName() == NAME_DateTime)
				{
					return MakeShared<FJsonValueString>(static_cast<const FDateTime*>(Data)->ToIso8601());
				}
			}
			return {};
		});

#undef LOCTEXT_NAMESPACE
