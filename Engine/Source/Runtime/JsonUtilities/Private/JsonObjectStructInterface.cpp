// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonObjectStructInterface.h"

#include "Misc/TransactionallySafeCriticalSection.h"
#include "Async/UniqueLock.h"

namespace JsonObjectStructInterface
{
	static FTransactionallySafeCriticalSection GStructConvertersMapCriticalSection;
	static TMap<const UScriptStruct*, const IJsonObjectStructConverter*> GStructConvertersMap;
}

void FJsonObjectStructInterfaceRegistry::RegisterStructConverter(const UScriptStruct* ScriptStruct, const IJsonObjectStructConverter* ConverterInterface)
{
	UE::TUniqueLock Lock(JsonObjectStructInterface::GStructConvertersMapCriticalSection);
	JsonObjectStructInterface::GStructConvertersMap.Add(ScriptStruct, ConverterInterface);
}

void FJsonObjectStructInterfaceRegistry::UnregisterStructConverter(const UScriptStruct* ScriptStruct)
{
	UE::TUniqueLock Lock(JsonObjectStructInterface::GStructConvertersMapCriticalSection);
	JsonObjectStructInterface::GStructConvertersMap.Remove(ScriptStruct);
}

bool FJsonObjectStructInterfaceRegistry::HasStructConverterRegistered(const UScriptStruct* ScriptStruct)
{
	return UE::Json::Private::GetStructConverterInterface(ScriptStruct) != nullptr;
}

namespace UE::Json::Private
{
	const IJsonObjectStructConverter* GetStructConverterInterface(const UScriptStruct* ScriptStruct)
	{
		UE::TUniqueLock Lock(JsonObjectStructInterface::GStructConvertersMapCriticalSection);
		return JsonObjectStructInterface::GStructConvertersMap.FindRef(ScriptStruct);
	}
}