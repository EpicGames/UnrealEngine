// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonObjectWrapper.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(JsonObjectWrapper)

FJsonObjectWrapper::FJsonObjectWrapper()
{
	JsonObject = MakeShared<FJsonObject>();
}

bool FJsonObjectWrapper::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
{
	// read JSON string from Buffer
	FString Json;
	if (*Buffer == TCHAR('"'))
	{
		int32 NumCharsRead = 0;
		if (!FParse::QuotedString(Buffer, Json, &NumCharsRead))
		{
			if (ErrorText)
			{
				ErrorText->Logf(ELogVerbosity::Warning, TEXT("FJsonObjectWrapper::ImportTextItem: Bad quoted string: %s\n"), Buffer);
			}
			
			return false;
		}
		Buffer += NumCharsRead;
	}
	else
	{
		// consume the rest of the string (this happens on Paste)
		Json = Buffer;
		Buffer += Json.Len();
	}

	// empty string resets/re-initializes shared pointer
	if (Json.IsEmpty())
	{
		JsonString.Empty();
		JsonObject = MakeShared<FJsonObject>();
		return true;
	}

	// parse the json
	if (!JsonObjectFromString(Json))
	{
		if (ErrorText)
		{
			ErrorText->Logf(ELogVerbosity::Warning, TEXT("FJsonObjectWrapper::ImportTextItem - Unable to parse json: %s\n"), *Json);
		}
		return false;
	}
	JsonString = Json;
	return true;
}

bool FJsonObjectWrapper::ExportTextItem(FString& ValueStr, FJsonObjectWrapper const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	// empty pointer yields empty string
	if (!JsonObject.IsValid())
	{
		ValueStr.Empty();
		return true;
	}

	// serialize the json
	return JsonObjectToString(ValueStr);
}

void FJsonObjectWrapper::PostSerialize(const FArchive& Ar)
{
	if (!JsonString.IsEmpty())
	{
		// try to parse JsonString
		if (!JsonObjectFromString(JsonString))
		{
			// do not abide a string that won't parse
			JsonString.Empty();
		}
	}
}

FJsonObjectWrapper::operator bool() const
{
	return JsonObject.IsValid() && !JsonObject->Values.IsEmpty();
}

bool FJsonObjectWrapper::JsonObjectToString(FString& Str) const
{
	TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Str, 0);
	return FJsonSerializer::Serialize(JsonObject.ToSharedRef(), JsonWriter, true);
}

bool FJsonObjectWrapper::JsonObjectFromString(const FString& Str)
{
	TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(Str);
	return FJsonSerializer::Deserialize(JsonReader, JsonObject);
}

SIZE_T FJsonObjectWrapper::GetAllocatedSize() const
{
	SIZE_T SizeBytes = 0;

	SizeBytes += JsonString.GetAllocatedSize();
	// NOTE - Given JsonObject is a shared ptr it's possible the underlying object is referenced by multiple things, 
	// so there's potential memory here could be getting counted multiple times by multiple wrappers
	SizeBytes += JsonObject.IsValid() ? JsonObject->GetMemoryFootprint() : 0;

	return SizeBytes;
}

