// Copyright Epic Games, Inc. All Rights Reserved. 

#include "JsonSchema/JsonSchemaEditorMetadata.h"

FJsonSchemaEditorMetadata::FJsonSchemaEditorMetadata()
{
}

FJsonSchemaEditorMetadata::FJsonSchemaEditorMetadata(const FJsonSchemaEditorMetadata& Other)
{
	*this = Other;
}

FJsonSchemaEditorMetadata& FJsonSchemaEditorMetadata::operator=(const FJsonSchemaEditorMetadata& Other)
{
	if (this != &Other)
	{
		RootStructMetadata = Other.RootStructMetadata;

		PropertyMemberPathToPropertyMetadataMap = Other.PropertyMemberPathToPropertyMetadataMap;

		CurrentPropertyMemberPath = Other.CurrentPropertyMemberPath;
	}
	
	return *this;
}

void FJsonSchemaEditorMetadata::SetPropertyMetadataForCurrentPropertyMemberPath(const TSharedRef<FJsonObject>& PropertyMetadata)
{
	PropertyMemberPathToPropertyMetadataMap.Add(*CurrentPropertyMemberPath, PropertyMetadata.ToSharedPtr());
}

TSharedPtr<FJsonObject> FJsonSchemaEditorMetadata::GetPropertyMetadataForCurrentPropertyMemberPath()
{
	if (TSharedPtr<FJsonObject>* Found = PropertyMemberPathToPropertyMetadataMap.Find(*CurrentPropertyMemberPath))
	{
		return *Found;
	}
	else
	{
		return nullptr;
	}
}

const FJsonSchemaEditorMetadata::FJsonSchemaPropertyMemberPathToPropertyMetadataMap& FJsonSchemaEditorMetadata::GetPropertyMemberPathToPropertyMetadataMap() const
{
	return PropertyMemberPathToPropertyMetadataMap;
}

FJsonSchemaEditorMetadata::FJsonSchemaPropertyMemberPathToPropertyMetadataMap& FJsonSchemaEditorMetadata::GetPropertyMemberPathToPropertyMetadataMap()
{
	return PropertyMemberPathToPropertyMetadataMap;
}
