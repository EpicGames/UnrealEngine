// Copyright Epic Games, Inc. All Rights Reserved. 

#include "JsonSchema/JsonSchemaGenerator.h"

#include "JsonDomBuilder.h"
#include "JsonObjectConverter.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "UObject/UnrealType.h"
#include "UObject/TextProperty.h"
#include "UObject/EnumProperty.h"
#include "UObject/StrProperty.h"
#include "UObject/Utf8StrProperty.h"
#include "UObject/PropertyOptional.h"
#include "JsonUtilitiesModularFeature.h"
#include "UObject/FieldPathProperty.h"
#include "JsonSchema/JsonSchemaVisitorStack.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/PropertyBag.h"

using namespace UE::JsonSchema;

// See all NOTE_JSON_SCHEMA_CHECK_FLAGS_NO_PARMS.
// A UStruct may be a UFunction, in which case a caller might have set the filter flags to ignore
// certain types of parameters (properties) of the function. After using the CheckFlags to check
// of a property/parameter should be ignored, we should stop considering flags related to function
// parameters (properties), by clearing all CPF_ParmFlags, so that they don't block schema
// generation (or metadata collection) for substructs/subproperties.

//
// Internal
//

namespace 
{
	template<class CharType, class PrintPolicy>
	FString JsonObjectToJsonObjectString(const TSharedRef<FJsonObject>& JsonObject, const int32 Indent)
	{
		FString OutString;
		
		TSharedRef<TJsonWriter<CharType, PrintPolicy>> JsonWriter = TJsonWriterFactory<CharType, PrintPolicy>::Create(&OutString, Indent);
		const bool bSuccess = FJsonSerializer::Serialize(JsonObject, JsonWriter);
		JsonWriter->Close();
		
		return (bSuccess ? OutString : TEXT(""));
	}
}

//
// Internal FJsonSchemaGeneratorVisitor
//

class FJsonSchemaGeneratorVisitor
{
public:

	explicit FJsonSchemaGeneratorVisitor(FJsonSchemaEditorMetadata& InEditorMetadata) :
		EditorMetadata(InEditorMetadata)
	{
	}
	
	TSharedPtr<FJsonObject> VisitRoot(const FVisitorTarget& VisitorTarget,
		const FJsonSchemaMemberPath& MemberPath, const FJsonSchemaPropertyFilter& PropertyFilter,
		const void* InstanceMemory)
	{
		const EVisitorStackElementFlags Flags = (VisitorTarget.IsType<FValidConstUStruct>() ?
			EVisitorStackElementFlags::IsRootStruct : EVisitorStackElementFlags::None);

		VisitorStack.Empty(); // Precaution.

		const FVisitorStackElement RootStackElement(VisitorTarget, Flags, MemberPath, PropertyFilter, MakeShared<FJsonObject>(), InstanceMemory);
		VisitorStack.Add(RootStackElement);

		while (!VisitorStack.IsEmpty())
		{
			const FVisitorStackElement StackElement = VisitorStack.Pop();

			if (StackElement.VisitorTarget.IsType<FValidConstUStruct>())
			{
				if (!VisitUStruct(StackElement.VisitorTarget.Get<FValidConstUStruct>(),
					StackElement.OutputSchema.ToSharedRef(), StackElement.MemberPath, StackElement.PropertyFilter,
					StackElement.Flags, StackElement.InstanceMemory))
				{
					return nullptr;
				}
			}
			else if (StackElement.VisitorTarget.IsType<FValidConstFProperty>())
			{
				if (!VisitFProperty(StackElement.VisitorTarget.Get<FValidConstFProperty>(),
					StackElement.OutputSchema.ToSharedRef(), StackElement.MemberPath, StackElement.PropertyFilter,
					StackElement.Flags, StackElement.InstanceMemory))
				{
					return nullptr;
				}
			}
		}

		return RootStackElement.OutputSchema;
	}
	
	// Dispatches a property to the appropriate type-specific schema builder, including the custom
	// callback. Returns true if the property was handled, false if the type is unsupported.
	// Does not check ArrayDim - callers are responsible for that.
	bool DispatchPropertyType(const FProperty& Property, const FJsonSchemaPropertyFilter& PropertyFilter,
		const TSharedRef<FJsonObject>& OutputSchema, const void* InstanceMemory)
	{
		bool ValidProperty = false;
		if (PropertyFilter.CustomCb && PropertyFilter.CustomCb->IsBound())
		{
			ValidProperty = PropertyFilter.CustomCb->Execute(&Property, "", OutputSchema);
		}

		if (ValidProperty)
		{
			// Do nothing because the CustomCb handled it.
		}
		else if (const FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(&Property))
		{
			ValidProperty = BuildJsonForOptionalProperty(*OptionalProperty, PropertyFilter, OutputSchema, InstanceMemory);
		}
		else if (const FStructProperty* StructProperty = CastField<FStructProperty>(&Property))
		{
			ValidProperty = BuildJsonForStructProperty(*StructProperty, PropertyFilter, OutputSchema, InstanceMemory);
		}
		else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(&Property))
		{
			// This will catch UObject* and UClass*.
			ValidProperty = BuildJsonForObjectProperty(*ObjectProperty, OutputSchema);
		}
		else if (const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(&Property))
		{
			// This will catch SoftObjectPtr and SoftClassPtr.
			ValidProperty = BuildJsonForObjectProperty(*SoftObjectProperty, OutputSchema);
		}
		else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(&Property))
		{
			ValidProperty = BuildJsonForArrayProperty(*ArrayProperty, PropertyFilter, OutputSchema, InstanceMemory);
		}
		else if (const FSetProperty* SetProperty = CastField<FSetProperty>(&Property))
		{
			ValidProperty = BuildJsonForSetProperty(*SetProperty, PropertyFilter, OutputSchema, InstanceMemory);
		}
		else if (const FMapProperty* MapProperty = CastField<FMapProperty>(&Property))
		{
			ValidProperty = BuildJsonForMapProperty(*MapProperty, PropertyFilter, OutputSchema, InstanceMemory);
		}
		else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(&Property))
		{
			ValidProperty = BuildJsonForEnumDefinition(EnumProperty->GetEnum(), OutputSchema);
		}
		else if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(&Property))
		{
			ValidProperty = BuildJsonForNumericProperty(*NumericProperty, OutputSchema);
		}
		else if (CastField<FBoolProperty>(&Property))
		{
			ValidProperty = BuildJsonForBoolProperty(OutputSchema);
		}
		else if (CastField<FStrProperty>(&Property) ||
			CastField<FTextProperty>(&Property) ||
			CastField<FNameProperty>(&Property) ||
			CastField<FUtf8StrProperty>(&Property))
		{
			ValidProperty = BuildJsonForStringProperty(OutputSchema);
		}

		return ValidProperty;
	}

	bool VisitFProperty(const FValidConstFProperty Property, const TSharedRef<FJsonObject>& OutputSchema,
		const FJsonSchemaMemberPath& MemberPath, const FJsonSchemaPropertyFilter& PropertyFilter,
		const EVisitorStackElementFlags Flags, const void* InstanceMemory)
	{
		EditorMetadata.CurrentPropertyMemberPath = MemberPath;

		// Fixed-size static arrays (e.g. UPROPERTY() int32 Values[3]) are not FArrayProperty — they
		// are a base property type with ArrayDim > 1. Handle them before DispatchPropertyType so the
		// custom callback still runs for the inner item type inside BuildJsonForStaticArrayProperty.
		const bool ValidProperty = Property->ArrayDim > 1
			? BuildJsonForStaticArrayProperty(*Property, PropertyFilter, OutputSchema, Flags, InstanceMemory)
			: DispatchPropertyType(*Property, PropertyFilter, OutputSchema, InstanceMemory);

		if (!ValidProperty)
		{
			// This means we've hit an unsupported property, which is either an error or a warning depending on
			// whether all properties are required or not. Generally all properties are only required for UFunctions.
			const FString Message = FString::Printf(TEXT("Property \"%s\" type %s unhandled during Json schema generation."),
				*Property->GetName(), *Property->GetCPPType());
			if ((static_cast<uint8>(Flags) & static_cast<uint8>(EVisitorStackElementFlags::RequireAllProperties)) == 0)
			{
				UE_LOGF(LogJson, Warning, "%ls", *Message);
				return true;
			}
			else
			{
				UE_LOGF(LogJson, Error, "%ls", *Message);
				return false;
			}
		}

		// Add all property metadata, if any.
		if (const TSharedPtr<FJsonObject> PropertyMetadataJson = EditorMetadata.GetPropertyMetadataForCurrentPropertyMemberPath())
		{
			for (const TTuple<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Tuple : PropertyMetadataJson->Values)
			{
				const FString JsonMemberName = PropertyFilter.PropertyAuthoredNameToJsonKey(FString(Tuple.Key));
				OutputSchema->SetField(*JsonMemberName, Tuple.Value);
			}
		}

		if (static_cast<uint8>(Flags) & static_cast<uint8>(EVisitorStackElementFlags::SkipDescription))
		{
			OutputSchema->RemoveField(TEXT("description"));
		}
		
		return true;
	}
	
	bool VisitUStruct(const FValidConstUStruct Struct, const TSharedRef<FJsonObject>& OutputSchema,
		const FJsonSchemaMemberPath& MemberPath, const FJsonSchemaPropertyFilter& PropertyFilter,
		const EVisitorStackElementFlags Flags, const void* InstanceMemory)
	{
		EditorMetadata.CurrentPropertyMemberPath = MemberPath;

		const bool bUseJsonRpcFormat = Struct->IsA(UFunction::StaticClass());
		
		// Only include "type": "object" in this top-level schema if it's not a function, and so not following JSON RPC format.
		if (!bUseJsonRpcFormat)
		{
			OutputSchema->SetStringField(TEXT("type"), TEXT("object"));
			OutputSchema->SetStringField(TEXT("title"), Struct->GetName());
		}

		// Special case handling for FJsonObjectWrapper, where we know this must be a Json object.   
		if (Struct == FJsonObjectWrapper::StaticStruct())
		{
			return true;
		}

		// Add all struct metadata.
		if (EnumHasAnyFlags(Flags, EVisitorStackElementFlags::IsRootStruct) && EditorMetadata.RootStructMetadata)
		{
			for (const TTuple<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Tuple : EditorMetadata.RootStructMetadata->Values)
			{
				const FString JsonMemberName = PropertyFilter.PropertyAuthoredNameToJsonKey(FString(Tuple.Key));
				OutputSchema->SetField(JsonMemberName, Tuple.Value);
			}
		}

		// If property is to be included - Adds property to JSON and possibly the required list, and also adds to the visitor stack for processing.
		// Returns whether the property was included.
		auto MaybeIncludeProperty = [this, PropertyFilter, Flags, bUseJsonRpcFormat, InstanceMemory](const FProperty* Property,
			const TSharedRef<FJsonObject>& PropertiesSchema, TArray<TSharedPtr<FJsonValue>>& RequiredPropertiesArray) -> bool
			{
				const FString PropertyName = Property->GetAuthoredName();
			
				UE_JSON_SCHEMA_SCOPED_MEMBER_PATH_PUSH(EditorMetadata.CurrentPropertyMemberPath, PropertyName, 
					!EnumHasAnyFlags(Flags, EVisitorStackElementFlags::IsRootStruct));

				if (PropertyFilter.IsPropertyIgnored(Property) || 
					PropertyFilter.IsPropertyMemberPathSkipped(*EditorMetadata.CurrentPropertyMemberPath))
				{
					return false;
				}

				// See all NOTE_JSON_SCHEMA_CHECK_FLAGS_NO_PARMS.
				FJsonSchemaPropertyFilter NewPropertyFilter = PropertyFilter;
				NewPropertyFilter.CheckFlags &= ~CPF_ParmFlags;
			
				const TSharedRef<FJsonObject> PropertySchema = MakeShared<FJsonObject>();
				
				const FString JsonMemberName = PropertyFilter.PropertyAuthoredNameToJsonKey(PropertyName); 
				PropertiesSchema->SetObjectField(JsonMemberName, PropertySchema);
				
				const void* PropertyInstanceMemory = InstanceMemory
					? static_cast<const uint8*>(InstanceMemory) + Property->GetOffset_ForInternal()
					: nullptr;

				VisitorStack.Emplace(
					FVisitorTarget(TInPlaceType<FValidConstFProperty>(), Property),
					bUseJsonRpcFormat ? EVisitorStackElementFlags::RequireAllProperties : EVisitorStackElementFlags::None,
					EditorMetadata.CurrentPropertyMemberPath,
					NewPropertyFilter,
					PropertySchema,
					PropertyInstanceMemory);

				const TSharedPtr<FJsonObject> PropertyMetadata = EditorMetadata.GetPropertyMetadataForCurrentPropertyMemberPath();

				if (PropertyFilter.IsPropertyMemberPathRequired(*EditorMetadata.CurrentPropertyMemberPath))
				{
					// Property has been explicitly required.
					RequiredPropertiesArray.Add(MakeShared<FJsonValueString>(JsonMemberName));
				}
				else if (!(PropertyMetadata && PropertyMetadata->HasField(TEXT("default"))) &&
					!Property->IsA<FOptionalProperty>())
				{
					// Property has not been explicitly required.
					// Only require non-optional properties that don't have default values.
					RequiredPropertiesArray.Add(MakeShared<FJsonValueString>(JsonMemberName));
				}
				
				return true;
			};
		
		// Properties & required.
		// If this struct is a function, use JSON RPC format (with inputSchema/outputSchema). 
		// Otherwise, use standard format.
		if (bUseJsonRpcFormat)
		{
			// If struct is a function, output JSON RPC -
			// https://github.com/modelcontextprotocol/modelcontextprotocol/blob/main/schema/2025-06-18/schema.ts#L928
			// Extension for 'name' -
			// https://github.com/modelcontextprotocol/modelcontextprotocol/blob/main/schema/2025-06-18/schema.ts#L312
			
			// NOTE - We know we are a UFunction, so will have a valid Outer.
			OutputSchema->SetStringField(TEXT("name"), *(Struct->GetOuter()->GetName() + "." + Struct->GetAuthoredName()));
			
			// Collect input and output properties.
			const TSharedRef<FJsonObject> FunctionInputPropertiesSchema = MakeShared<FJsonObject>();
			TArray<TSharedPtr<FJsonValue>> FunctionInputRequiredPropertiesArray;
			const TSharedRef<FJsonObject> FunctionOutputPropertiesSchema = MakeShared<FJsonObject>();
			TArray<TSharedPtr<FJsonValue>> FunctionOutputRequiredPropertiesArray;
			for (const FProperty* Property : TFieldRange<FProperty>(Struct))
			{
				if (Property->HasAnyPropertyFlags(CPF_ReturnParm))
				{
					MaybeIncludeProperty(Property, FunctionOutputPropertiesSchema, FunctionOutputRequiredPropertiesArray);
				}
				else
				{
					MaybeIncludeProperty(Property, FunctionInputPropertiesSchema, FunctionInputRequiredPropertiesArray);
				}
			}
	
			// Input schema is not optional.
			{
				const TSharedPtr<FJsonObject> FunctionInputSchema = MakeShared<FJsonObject>();
				FunctionInputSchema->SetStringField(TEXT("type"), TEXT("object"));
				OutputSchema->SetObjectField(TEXT("inputSchema"), FunctionInputSchema);
				
				if (!FunctionInputPropertiesSchema->Values.IsEmpty())
				{
					FunctionInputSchema->SetObjectField(TEXT("properties"), FunctionInputPropertiesSchema);

					if (!FunctionInputRequiredPropertiesArray.IsEmpty())
					{
						FunctionInputSchema->SetArrayField(TEXT("required"), FunctionInputRequiredPropertiesArray);
					}
				}
			}
	
			// Output schema is optional.
			if (!FunctionOutputPropertiesSchema->Values.IsEmpty())
			{
				const TSharedPtr<FJsonObject> FunctionOutputSchema = MakeShared<FJsonObject>();
				FunctionOutputSchema->SetStringField(TEXT("type"), TEXT("object"));
				OutputSchema->SetObjectField(TEXT("outputSchema"), FunctionOutputSchema);
				
				// We have OutputSchemaPropertiesJson (see above)
				{
					FunctionOutputSchema->SetObjectField(TEXT("properties"), FunctionOutputPropertiesSchema);
					
					if (!FunctionOutputRequiredPropertiesArray.IsEmpty())
					{
						FunctionOutputSchema->SetArrayField(TEXT("required"), FunctionOutputRequiredPropertiesArray);
					}
				}
			}
		}
		else
		{
			// If struct is not a function, output standard format.
				
			const TSharedRef<FJsonObject> PropertiesSchema = MakeShared<FJsonObject>();
			TArray<TSharedPtr<FJsonValue>> RequiredPropertiesArray;
			for (const FProperty* Property : TFieldRange<FProperty>(Struct))
			{
				MaybeIncludeProperty(Property, PropertiesSchema, RequiredPropertiesArray);
			}
	
			if (!PropertiesSchema->Values.IsEmpty())
			{
				OutputSchema->SetObjectField(TEXT("properties"), PropertiesSchema);
				if (!RequiredPropertiesArray.IsEmpty())
				{
					OutputSchema->SetArrayField(TEXT("required"), RequiredPropertiesArray);
				}
			}
		}
		
		return true;
	}
	
private:
	
	static bool BuildJsonForEnumDefinition(const UEnum* EnumDef, const TSharedRef<FJsonObject>& OutputSchema)
	{
		if (!EnumDef)
		{
			return false;
		}

		FJsonDomBuilder::FArray EnumNames;
		const int32 NumEnum = EnumDef->NumEnums();
		for (int32 EnumIndex = 0; EnumIndex < NumEnum; ++EnumIndex)
		{
			const FString StringValue = EnumDef->GetAuthoredNameStringByIndex(EnumIndex);
			if (StringValue.EndsWith(TEXT("_MAX"), ESearchCase::CaseSensitive))
			{
				continue;
			}
			EnumNames.Add(StringValue);
		}

		OutputSchema->SetStringField(TEXT("type"), TEXT("string"));
		OutputSchema->SetStringField(TEXT("title"), EnumDef->GetName());
		OutputSchema->SetField(TEXT("enum"), EnumNames.AsJsonValue());
		return true;
	}

	bool BuildJsonForOptionalProperty(const FOptionalProperty& OptionalProperty,
		const FJsonSchemaPropertyFilter& PropertyFilter, const TSharedRef<FJsonObject>& OutputSchema,
		const void* InstanceMemory)
	{
		// If the optional is set, pass the inner value pointer so that runtime-typed properties
		// (e.g. FInstancedStruct) inside a TOptional can produce an accurate schema.
		// GetValuePointerForReadIfSet returns nullptr when the optional is unset, which is the
		// correct fallback (the inner property visitor will use a static/type-only schema).
		const void* InnerInstanceMemory = InstanceMemory
			? OptionalProperty.GetValuePointerForReadIfSet(InstanceMemory)
			: nullptr;

		VisitorStack.Emplace(
			FVisitorTarget(TInPlaceType<FValidConstFProperty>(), OptionalProperty.GetValueProperty()),
			EVisitorStackElementFlags::None,
			EditorMetadata.CurrentPropertyMemberPath,
			PropertyFilter,
			OutputSchema,
			InnerInstanceMemory);
		return true;
	}

	bool BuildJsonForStructProperty(const FStructProperty& StructProperty,
		const FJsonSchemaPropertyFilter& PropertyFilter, const TSharedRef<FJsonObject>& OutputSchema,
		const void* InstanceMemory)
	{
		if (StructProperty.Struct == FInstancedStruct::StaticStruct())
		{
			// With instance memory, push the actual struct so VisitUStruct generates the full schema.
			if (InstanceMemory)
			{
				const FInstancedStruct* InstancedStruct = static_cast<const FInstancedStruct*>(InstanceMemory);
				if (InstancedStruct->IsValid())
				{
					VisitorStack.Emplace(
						FVisitorTarget(TInPlaceType<FValidConstUStruct>(), InstancedStruct->GetScriptStruct()),
						EVisitorStackElementFlags::None,
						EditorMetadata.CurrentPropertyMemberPath,
						PropertyFilter,
						OutputSchema,
						InstancedStruct->GetMemory());
					return true;
				}
			}
			// No instance or invalid struct: static fallback.
			OutputSchema->SetStringField(TEXT("type"), TEXT("object"));
			OutputSchema->SetStringField(TEXT("title"), TEXT("FInstancedStruct"));
			OutputSchema->SetStringField(TEXT("description"),
				TEXT("Polymorphic struct. Provide _structType with the UScriptStruct path (e.g. /Script/Module.StructName) to create from scratch."));
			TSharedPtr<FJsonObject> StructTypeSchema = MakeShared<FJsonObject>();
			StructTypeSchema->SetStringField(TEXT("type"), TEXT("string"));
			StructTypeSchema->SetStringField(TEXT("description"),
				TEXT("UScriptStruct path name for polymorphic type identification (e.g. /Script/Module.StructName)."));
			TSharedPtr<FJsonObject> Properties = MakeShared<FJsonObject>();
			Properties->SetObjectField(TEXT("_structType"), StructTypeSchema);
			OutputSchema->SetObjectField(TEXT("properties"), Properties);
			return true;
		}

		if (StructProperty.Struct == FInstancedPropertyBag::StaticStruct())
		{
			// With instance memory, push the actual UPropertyBag so VisitUStruct generates the full schema.
			if (InstanceMemory)
			{
				const FInstancedPropertyBag* Bag = static_cast<const FInstancedPropertyBag*>(InstanceMemory);
				if (Bag->IsValid())
				{
					if (const UPropertyBag* BagStruct = Bag->GetPropertyBagStruct())
					{
						VisitorStack.Emplace(
							FVisitorTarget(TInPlaceType<FValidConstUStruct>(), BagStruct),
							EVisitorStackElementFlags::None,
							EditorMetadata.CurrentPropertyMemberPath,
							PropertyFilter,
							OutputSchema,
							Bag->GetValue().GetMemory());
						return true;
					}
				}
			}
			// No instance or invalid bag: static fallback.
			OutputSchema->SetStringField(TEXT("type"), TEXT("object"));
			OutputSchema->SetStringField(TEXT("title"), TEXT("FInstancedPropertyBag"));
			return true;
		}

		VisitorStack.Emplace(
			FVisitorTarget(TInPlaceType<FValidConstUStruct>(), StructProperty.Struct),
			EVisitorStackElementFlags::None,
			EditorMetadata.CurrentPropertyMemberPath,
			PropertyFilter,
			OutputSchema,
			InstanceMemory);
		OutputSchema->SetStringField(TEXT("title"), StructProperty.Struct->GetName());
		return true;
	}

	bool BuildJsonForArrayProperty(const FArrayProperty& ArrayProperty,
		const FJsonSchemaPropertyFilter& PropertyFilter, const TSharedRef<FJsonObject>& OutputSchema,
		const void* InstanceMemory)
	{
		const TSharedRef<FJsonObject> ItemsJson = MakeShared<FJsonObject>();
		VisitorStack.Emplace(
			FVisitorTarget(TInPlaceType<FValidConstFProperty>(), ArrayProperty.Inner),
			EVisitorStackElementFlags::SkipDescription,
			EditorMetadata.CurrentPropertyMemberPath,
			PropertyFilter,
			ItemsJson,
			// No instance memory: a TArray can hold elements of different runtime types (e.g. each
			// FInstancedStruct element may have a distinct UScriptStruct), so there is no single
			// representative instance. The element visitor falls back to a static/type-only schema.
			nullptr);

		OutputSchema->SetStringField(TEXT("type"), TEXT("array"));
		OutputSchema->SetObjectField(TEXT("items"), ItemsJson);

		if (InstanceMemory && ArrayProperty.HasAnyPropertyFlags(CPF_EditFixedSize))
		{
			const int32 FixedSize = FScriptArrayHelper(&ArrayProperty, InstanceMemory).Num();
			if (FixedSize > 0)
			{
				OutputSchema->SetNumberField(TEXT("minItems"), FixedSize);
				OutputSchema->SetNumberField(TEXT("maxItems"), FixedSize);
			}
		}

		return true;
	}

	bool BuildJsonForStaticArrayProperty(const FProperty& Property,
		const FJsonSchemaPropertyFilter& PropertyFilter, const TSharedRef<FJsonObject>& OutputSchema,
		const EVisitorStackElementFlags Flags, const void* InstanceMemory)
	{
		const TSharedRef<FJsonObject> ItemsJson = MakeShared<FJsonObject>();
		if (!DispatchPropertyType(Property, PropertyFilter, ItemsJson, InstanceMemory))
		{
			const FString Message = FString::Printf(
				TEXT("Property \"%s\" element type %s unhandled during Json schema generation for fixed-size array."),
				*Property.GetName(), *Property.GetCPPType());
			if (!EnumHasAnyFlags(Flags, EVisitorStackElementFlags::RequireAllProperties))
			{
				UE_LOG(LogJson, Warning, TEXT("%s"), *Message);
			}
			else
			{
				UE_LOG(LogJson, Error, TEXT("%s"), *Message);
				return false;
			}
		}

		OutputSchema->SetStringField(TEXT("type"), TEXT("array"));
		OutputSchema->SetObjectField(TEXT("items"), ItemsJson);
		OutputSchema->SetNumberField(TEXT("minItems"), Property.ArrayDim);
		OutputSchema->SetNumberField(TEXT("maxItems"), Property.ArrayDim);
		return true;
	}

	bool BuildJsonForSetProperty(const FSetProperty& SetProperty,
		const FJsonSchemaPropertyFilter& PropertyFilter, const TSharedRef<FJsonObject>& OutputSchema,
		const void* InstanceMemory)
	{
		const TSharedRef<FJsonObject> ItemsJson = MakeShared<FJsonObject>();
		VisitorStack.Emplace(
			FVisitorTarget(TInPlaceType<FValidConstFProperty>(), SetProperty.ElementProp),
			EVisitorStackElementFlags::SkipDescription,
			EditorMetadata.CurrentPropertyMemberPath,
			PropertyFilter,
			ItemsJson,
			// No instance memory: same reasoning as BuildJsonForArrayProperty - TSet has no
			// single representative element to drive runtime-typed schema generation.
			nullptr);

		OutputSchema->SetStringField(TEXT("type"), TEXT("array"));
		OutputSchema->SetBoolField(TEXT("uniqueItems"), true);
		OutputSchema->SetObjectField(TEXT("items"), ItemsJson);

		if (InstanceMemory && SetProperty.HasAnyPropertyFlags(CPF_EditFixedSize))
		{
			const int32 FixedSize = FScriptSetHelper(&SetProperty, InstanceMemory).Num();
			if (FixedSize > 0)
			{
				OutputSchema->SetNumberField(TEXT("minItems"), FixedSize);
				OutputSchema->SetNumberField(TEXT("maxItems"), FixedSize);
			}
		}

		return true;
	}

	bool BuildJsonForMapProperty(const FMapProperty& MapProperty,
		const FJsonSchemaPropertyFilter& PropertyFilter, const TSharedRef<FJsonObject>& OutputSchema,
		const void* InstanceMemory)
	{
		if (MapProperty.KeyProp->IsA<FStrProperty>() || MapProperty.KeyProp->IsA<FNameProperty>())
		{
			const TSharedRef<FJsonObject> MapValuesJson = MakeShared<FJsonObject>();
			VisitorStack.Emplace(
				FVisitorTarget(TInPlaceType<FValidConstFProperty>(), MapProperty.ValueProp),
				EVisitorStackElementFlags::SkipDescription,
				EditorMetadata.CurrentPropertyMemberPath,
				PropertyFilter,
				MapValuesJson,
				// No instance memory: same reasoning as BuildJsonForArrayProperty - TMap has no
				// single representative element to drive runtime-typed schema generation.
				nullptr);

			OutputSchema->SetStringField(TEXT("type"), TEXT("object"));
			OutputSchema->SetObjectField(TEXT("additionalProperties"), MapValuesJson);

			if (InstanceMemory && MapProperty.HasAnyPropertyFlags(CPF_EditFixedSize))
			{
				const int32 FixedSize = FScriptMapHelper(&MapProperty, InstanceMemory).Num();
				if (FixedSize > 0)
				{
					OutputSchema->SetNumberField(TEXT("minProperties"), FixedSize);
					OutputSchema->SetNumberField(TEXT("maxProperties"), FixedSize);
				}
			}

			return true;
		}
		return false;
	}

	bool BuildJsonForObjectProperty(const FObjectPropertyBase& ObjectProperty, const TSharedRef<FJsonObject>& OutputSchema)
	{
		// This mirrors the behavior of JsonObjectConverter which exports instance objects by value instead of by reference.
		if (ObjectProperty.HasAnyPropertyFlags(CPF_PersistentInstance))
		{
			OutputSchema->SetStringField(TEXT("type"), TEXT("object"));
		}
		else
		{
			OutputSchema->SetStringField(TEXT("type"), TEXT("string"));
		}
		OutputSchema->SetStringField(TEXT("title"), ObjectProperty.PropertyClass.GetName());
		return true;
	}

	static bool BuildJsonForNumericProperty(const FNumericProperty& NumericProperty, const TSharedRef<FJsonObject>& OutputSchema)
	{
		if (const UEnum* EnumDef = NumericProperty.GetIntPropertyEnum())
		{
			return BuildJsonForEnumDefinition(EnumDef, OutputSchema);
		}
		else if (NumericProperty.IsFloatingPoint())
		{
			OutputSchema->SetStringField(TEXT("type"), TEXT("number"));
			return true;
		}
		else if (NumericProperty.IsInteger())
		{
			OutputSchema->SetStringField(TEXT("type"), TEXT("integer"));
			return true;
		}
		return false;
	}

	static bool BuildJsonForBoolProperty(const TSharedRef<FJsonObject>& OutputSchema)
	{
		OutputSchema->SetStringField(TEXT("type"), TEXT("boolean"));
		return true;
	}

	static bool BuildJsonForStringProperty(const TSharedRef<FJsonObject>& OutputSchema)
	{
		OutputSchema->SetStringField(TEXT("type"), TEXT("string"));
		return true;
	}

	FVisitorStack VisitorStack;
	FJsonSchemaEditorMetadata& EditorMetadata; 	
};

//
// FJsonSchemaGenerator
//
// See all NOTE_JSON_UTILITIES_SCHEMA_GENERATOR_METADATA_MODULAR_FEATURE.
// If a Modular Feature is registered, for collecting struct metadata, call it to get
// metadata. Collecting metadata from structs and their properties can happen in the
// Editor context, where the Modular Feature will be registered. (If not running in
// Editor context, no metadata can be collected, and the Modular Feature will not be
// registered.)
//

/*static*/ TSharedPtr<FJsonObject> FJsonSchemaGenerator::FPropertyToJsonSchemaObject(TNotNull<const FProperty*> Property,
	const FJsonSchemaPropertyFilter& PropertyFilter, const FJsonSchemaEditorMetadata* CachedEditorMetadata,
	const void* InstanceMemory)
{
	// See all NOTE_JSON_UTILITIES_SCHEMA_GENERATOR_METADATA_MODULAR_FEATURE.
	FJsonSchemaEditorMetadata EditorMetadata;
	if (CachedEditorMetadata)
	{
		EditorMetadata = *const_cast<FJsonSchemaEditorMetadata*>(CachedEditorMetadata);
	}
	else if (IJsonUtilitiesModularFeature* ModuleFeature = IJsonUtilitiesModularFeature::Get())
	{
		EditorMetadata = ModuleFeature->FPropertyToJsonSchemaMetadata(Property, PropertyFilter, InstanceMemory);
	}

	FJsonSchemaGeneratorVisitor JsonSchemaGeneratorVisitor(EditorMetadata);
	return JsonSchemaGeneratorVisitor.VisitRoot(
		FVisitorTarget(TInPlaceType<FValidConstFProperty>(), Property),
		FJsonSchemaMemberPath(*Property->GetAuthoredName()),
		PropertyFilter,
		InstanceMemory);
}

/*static*/ FString FJsonSchemaGenerator::FPropertyToJsonSchemaString(TNotNull<const FProperty*> Property,
	const FJsonSchemaPropertyFilter& PropertyFilter, const FJsonSchemaEditorMetadata* CachedEditorMetadata,
	const int32 Indent, const bool bPrettyPrint)
{
	FString OutputSchemaString;
	
	if (const TSharedPtr<FJsonObject> JsonSchema = FPropertyToJsonSchemaObject(Property, PropertyFilter, CachedEditorMetadata))
	{
		if (bPrettyPrint)
		{
			OutputSchemaString = JsonObjectToJsonObjectString<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>(JsonSchema.ToSharedRef(), Indent);
		}
		else
		{
			OutputSchemaString = JsonObjectToJsonObjectString<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>(JsonSchema.ToSharedRef(), Indent);
		}
	}

	return OutputSchemaString;
}

/*static*/ TSharedPtr<FJsonObject> FJsonSchemaGenerator::UStructToJsonSchemaObject(TNotNull<const UStruct*> Struct,
	const FJsonSchemaPropertyFilter& PropertyFilter, const FJsonSchemaEditorMetadata* CachedEditorMetadata,
	const void* InstanceMemory)
{
	// See all NOTE_JSON_UTILITIES_SCHEMA_GENERATOR_METADATA_MODULAR_FEATURE.
	FJsonSchemaEditorMetadata EditorMetadata;
	if (CachedEditorMetadata)
	{
		EditorMetadata = *const_cast<FJsonSchemaEditorMetadata*>(CachedEditorMetadata);
	}
	else if (IJsonUtilitiesModularFeature* ModuleFeature = IJsonUtilitiesModularFeature::Get())
	{
		EditorMetadata = ModuleFeature->UStructToJsonSchemaMetadata(Struct, PropertyFilter, InstanceMemory);
	}

	FJsonSchemaGeneratorVisitor JsonSchemaGeneratorVisitor(EditorMetadata);
	return JsonSchemaGeneratorVisitor.VisitRoot(
		FVisitorTarget(TInPlaceType<FValidConstUStruct>(), Struct),
		FJsonSchemaMemberPath(),
		PropertyFilter,
		InstanceMemory);
}

/*static*/ FString FJsonSchemaGenerator::UStructToJsonSchemaString(TNotNull<const UStruct*> Struct,
	const FJsonSchemaPropertyFilter& PropertyFilter, const FJsonSchemaEditorMetadata* CachedEditorMetadata,
	const int32 Indent, const bool bPrettyPrint)
{
	FString OutputSchemaString;
	
	if (const TSharedPtr<FJsonObject> JsonSchema = UStructToJsonSchemaObject(Struct, PropertyFilter, CachedEditorMetadata))
	{
		if (bPrettyPrint)
		{
			OutputSchemaString = JsonObjectToJsonObjectString<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>(JsonSchema.ToSharedRef(), Indent);
		}
		else
		{
			OutputSchemaString = JsonObjectToJsonObjectString<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>(JsonSchema.ToSharedRef(), Indent);
		}
	}

	return OutputSchemaString;
}
