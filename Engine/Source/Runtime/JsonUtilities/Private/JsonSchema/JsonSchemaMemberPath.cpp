// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonSchema/JsonSchemaMemberPath.h"

//
// FJsonSchemaMemberPath
//

/*static*/ const FStringView FJsonSchemaMemberPath::Delimiter = TEXTVIEW(".");

FJsonSchemaMemberPath::FJsonSchemaMemberPath()
{
}

FJsonSchemaMemberPath::FJsonSchemaMemberPath(const FJsonSchemaMemberPath& InMemberPath)
	: PathString(InMemberPath.PathString)
{
}

FJsonSchemaMemberPath::FJsonSchemaMemberPath(FStringView InitialString)
	: PathString(InitialString)
{
}

FJsonSchemaMemberPath& FJsonSchemaMemberPath::operator=(const FJsonSchemaMemberPath& Other)
{
	if (this != &Other)
	{
		PathString = Other.PathString;
	}

	return *this;
}

bool FJsonSchemaMemberPath::Push(FStringView String)
{
	if (String.IsEmpty())
	{
		return false;
	}

	PathString.Append(String);

	return true;
}

bool FJsonSchemaMemberPath::Pop(FStringView String)
{
	if (String.IsEmpty())
	{
		return false;
	}

	// Safety check. Make sure current property path indeed ends with the text the caller wants to trim from the end.
	{
		const int32 Offset = Len() - String.Len();
		if (!ensureAlways(Offset >= 0))
		{
			return false;
		}

		if (!ensureAlways(String.Equals(GetData() + Offset, ESearchCase::CaseSensitive)))
		{
			return false;
		}
	}

	PathString.LeftInline(Len() - String.Len());

	return true;
}

bool FJsonSchemaMemberPath::PushSubmember(FStringView SubmemberString)
{
	if (SubmemberString.IsEmpty())
	{
		return false;
	}

	return (Push(Delimiter) && Push(SubmemberString));
}

bool FJsonSchemaMemberPath::PopSubmember(FStringView SubmemberString)
{
	if (SubmemberString.IsEmpty())
	{
		return false;
	}

	return (Pop(SubmemberString) && Pop(Delimiter));
}

FText FJsonSchemaMemberPath::ToText() const
{
	return FText::FromString(PathString);
}
