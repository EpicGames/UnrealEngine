// Copyright Epic Games, Inc. All Rights Reserved. 

#include "JsonSchema/JsonSchemaPropertyFilter.h"

FJsonSchemaPropertyFilter::FJsonSchemaPropertyFilter(
	const EPropertyFlags InCheckFlags,
	const EPropertyFlags InSkipFlags,
	const TOptional<TSet<FString>>& InRequiredPropertyMemberPaths,
	const TOptional<TSet<FString>>& InSkipPropertyMemberPaths,
	const EJsonObjectConversionFlags InConversionFlags,
	CustomCallback* InCustomCb)
	:
	CheckFlags(InCheckFlags),
	SkipFlags(InSkipFlags),
	RequiredPropertyMemberPaths(InRequiredPropertyMemberPaths),
	SkipPropertyMemberPaths(InSkipPropertyMemberPaths),
	ConversionFlags(InConversionFlags),
	CustomCb(InCustomCb)
{
}

FJsonSchemaPropertyFilter::FJsonSchemaPropertyFilter(const FJsonSchemaPropertyFilter& Other)
{
	CheckFlags = Other.CheckFlags;
	SkipFlags = Other.SkipFlags;
	RequiredPropertyMemberPaths = Other.RequiredPropertyMemberPaths;
	SkipPropertyMemberPaths = Other.SkipPropertyMemberPaths;
	ConversionFlags = Other.ConversionFlags;
	CustomCb = Other.CustomCb;
}

bool FJsonSchemaPropertyFilter::IsPropertyIgnored(TNotNull<const FProperty*> Property) const
{
	return (Property->HasAnyPropertyFlags(SkipFlags) || (CheckFlags != CPF_None && !Property->HasAnyPropertyFlags(CheckFlags)));
}

bool FJsonSchemaPropertyFilter::IsPropertyMemberPathRequired(const FString& PropertyMemberPath) const
{
	if (PropertyMemberPath.IsEmpty())
	{
		return false;
	}
	
	return (RequiredPropertyMemberPaths.IsSet() && RequiredPropertyMemberPaths->Contains(PropertyMemberPath));
}

bool FJsonSchemaPropertyFilter::IsPropertyMemberPathSkipped(const FString& PropertyMemberPath) const
{
	if (PropertyMemberPath.IsEmpty())
	{
		return false;
	}
	
	return (SkipPropertyMemberPaths.IsSet() && SkipPropertyMemberPaths->Contains(PropertyMemberPath));
}

FString FJsonSchemaPropertyFilter::PropertyAuthoredNameToJsonKey(const FString& PropertyAuthoredName) const
{
	// TODO - FJsonObjectConverter::StandardizeCase() leaves a lot to be desired. We should look into writing a new one.
	return (EnumHasAnyFlags(ConversionFlags, EJsonObjectConversionFlags::SkipStandardizeCase) ?
		PropertyAuthoredName :
		FJsonObjectConverter::StandardizeCase(PropertyAuthoredName));
}
