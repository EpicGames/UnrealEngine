// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "UObject/UnrealType.h"
#include "Misc/NotNull.h"
#include "Misc/TVariant.h"
#include "Templates/SharedPointer.h"
#include "Dom/JsonObject.h"
#include "JsonSchema/JsonSchemaMemberPath.h"
#include "JsonSchema/JsonSchemaPropertyFilter.h"

namespace UE::JsonSchema
{
	using FValidConstUStruct = TNotNull<const UStruct*>;
	using FValidConstFProperty = TNotNull<const FProperty*>;
	using FVisitorTarget = TVariant<FValidConstUStruct, FValidConstFProperty>;
	
	enum class EVisitorStackElementFlags : uint8
	{
		None					= 0,
		IsRootStruct			= 1 << 0,
		RequireAllProperties	= 1 << 1,
		SkipDescription			= 1 << 2
	};
	ENUM_CLASS_FLAGS(EVisitorStackElementFlags);

	struct FVisitorStackElement
	{
		explicit FVisitorStackElement(const FVisitorTarget& InVisitorTarget,
				const EVisitorStackElementFlags InFlags,
				const FJsonSchemaMemberPath& InMemberPath,
				const FJsonSchemaPropertyFilter& InPropertyFilter,
				const TSharedPtr<FJsonObject>& InOutputSchema,
				const void* InInstanceMemory) :
			VisitorTarget(InVisitorTarget),
			Flags(InFlags),
			MemberPath(InMemberPath),
			PropertyFilter(InPropertyFilter),
			OutputSchema(InOutputSchema),
			InstanceMemory(InInstanceMemory)
		{
			if (VisitorTarget.IsType<FValidConstFProperty>())
			{
				check(!EnumHasAnyFlags(Flags, EVisitorStackElementFlags::IsRootStruct));
			}
		}

		FVisitorTarget VisitorTarget;
		EVisitorStackElementFlags Flags = EVisitorStackElementFlags::None;
		FJsonSchemaMemberPath MemberPath;
		FJsonSchemaPropertyFilter PropertyFilter;
		const TSharedPtr<FJsonObject> OutputSchema;
		/** Optional pointer to the live instance for this target. Used to generate accurate schemas for
		 *  types whose structure is only known at runtime (e.g. FInstancedStruct, FInstancedPropertyBag). */
		const void* InstanceMemory = nullptr;
	};

	using FVisitorStack = TArray<FVisitorStackElement>; 
}
