// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Build.h"

#if !UE_BUILD_SHIPPING

#include "HAL/IConsoleManager.h"
#include "HAL/PlatformStackWalk.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/SecureHash.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/JsonWriter.h"
#include "String/ParseTokens.h"
#include "UObject/Class.h"
#include "UObject/EnumProperty.h"
#include "UObject/Field.h"
#include "UObject/MetaData.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectIterator.h"
#include "VerseVM/VVMClass.h"
#include "VerseVM/VVMEnumeration.h"
#include "VerseVM/VVMValue.h"
#include "VerseVM/VVMVerseClass.h"
#include "VerseVM/VVMVerseEnum.h"
#include "VerseVM/VVMVerseFunction.h"
#include "VerseVM/VVMVerseStruct.h"

namespace UE::JsonUtilities
{

class FArchiveMD5Generator : public FArchiveUObject
{
	FMD5 HashBuilder;

public:
	FArchiveMD5Generator()
	{
		SetIsLoading(false);
		SetIsSaving(true);
		//SetIsPersistent(true);
	}

	virtual FArchive& operator<<(FName& Name) override
	{
		FString SavedString(Name.ToString());
		*this << SavedString;
		return *this;
	}

	//~ Begin FArchive Interface
	virtual void Serialize(void* Data, int64 Length) override
	{
		HashBuilder.Update(static_cast<uint8*>(Data), Length);
	}

	virtual FString GetArchiveName() const override { return TEXT("UE::JsonUtilities::FArchiveMD5Generator"); }
	//~ End FArchive Interface

	FMD5Hash GetMD5()
	{
		FMD5Hash Result;
		Result.Set(HashBuilder);
		return Result;
	}
};

class FJsonUETypeWriter : public TJsonStringWriter<>
{
public:
	explicit FJsonUETypeWriter(FString* Out) : TJsonStringWriter(Out, 0)
	{
	}
};


template<typename T, typename GET_NAME_FUNC, typename WRITE_FUNC>
void WriteSortedObjectHelper(const TCHAR* PropertyName, const TArray<T>& Array, FJsonUETypeWriter& Writer, GET_NAME_FUNC && GetNameFunc, WRITE_FUNC && WriteFunc)
{
	if (Array.IsEmpty())
	{
		return;
	}

	Writer.WriteObjectStart(PropertyName);

	TMap<FString, int32> NameToIndex;
	for (int32 Index = 0; Index < Array.Num(); ++Index)
	{
		NameToIndex.Add(GetNameFunc(Array[Index]), Index);
	}

	TArray<FString> Keys;
	NameToIndex.GetKeys(Keys);
	Keys.Sort();

	for (const FString& Key : Keys)
	{

		Writer.WriteObjectStart(Key);
		const int32 Index = *NameToIndex.Find(Key);
		WriteFunc(Array[Index]);
		Writer.WriteObjectEnd();
	}

	Writer.WriteObjectEnd();
}

// a simple way to dissect a flag bits array
template<typename EnumType>
FString UEnumFlagsToString(EnumType Flags)
{	
	//TArray<int8> Result;
	FString Result = UEnum::GetValueAsString(Flags);

	// only assume flags if we don't match one
	if (Result.IsEmpty() || Result == TEXT("None"))
	{
		for (int8 Idx = 0; Idx < 64; ++Idx)
		{
			const EnumType Flag = EnumType(1ull<<Idx);

			if (uint64(Flag) & uint64(Flags))
			{
				const FString FlagValue = UEnum::GetValueAsString(Flag);

				if (!FlagValue.IsEmpty())
				{
					if (!Result.IsEmpty())
					{
						Result += TEXT(" | ");
					}

					Result.Append(FlagValue);
				}
			}
		}
	}

	Result.Append(FString::Printf(TEXT(" (0x%llx)"), uint64(Flags)));

	return Result;
}

template<typename T>
TArray<int8> FindUsedBits(T Flags)
{
	TArray<int8> Result;

	for (int8 Idx = 0; Idx < 64; ++Idx)
	{
		if (uint64(Flags) & (1ull<<Idx))
		{
			Result.Add(Idx);
		}
	}

	return Result;
}

void WriteFScriptSparseArrayLayout(const FScriptSparseArrayLayout& SparseArrayLayout, FJsonUETypeWriter& Writer)
{
	Writer.WriteValue(TEXT("Alignment"), SparseArrayLayout.Alignment);
	Writer.WriteValue(TEXT("Size"), SparseArrayLayout.Size);
}

void WriteFScriptSetLayout(const FScriptSetLayout& ScriptSetLayout, FJsonUETypeWriter& Writer)
{
	Writer.WriteValue(TEXT("Size"), ScriptSetLayout.Size);

#if UE_USE_COMPACT_SET_AS_DEFAULT
	Writer.WriteValue(TEXT("Alignment"), ScriptSetLayout.Alignment);
#else
	Writer.WriteValue(TEXT("HashNextIdOffset"), ScriptSetLayout.HashNextIdOffset);
	Writer.WriteValue(TEXT("HashIndexOffset"), ScriptSetLayout.HashIndexOffset);

	Writer.WriteObjectStart(TEXT("SparseArrayLayout"));
	WriteFScriptSparseArrayLayout(ScriptSetLayout.SparseArrayLayout, Writer);
	Writer.WriteObjectEnd();
#endif
}

void WriteFScriptMapLayout(const FScriptMapLayout& MapLayout, FJsonUETypeWriter& Writer)
{
	Writer.WriteValue(TEXT("ValueOffset"), MapLayout.ValueOffset);
	Writer.WriteObjectStart(TEXT("SetLayout"));
	WriteFScriptSetLayout(MapLayout.SetLayout, Writer);
	Writer.WriteObjectEnd();
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)

// The only way I can find a way to get these to compile is the 
template<typename T>
void WriteVerseVClass(const TCHAR* Name, const T& Class, FJsonUETypeWriter& Writer)
{
	// TODO: how to access this
}

template<typename T>
void WriteVerseVShape(const TCHAR* Name, const T& Shape, FJsonUETypeWriter& Writer)
{
	// TODO: how to access this
}

void WriteVerseVFunction(const TCHAR* Name, const Verse::VFunction& Value, FJsonUETypeWriter& Writer)
{
	// TODO: how to access this
}

void WriteVerseVEnumeration(const TCHAR* Name, const Verse::VEnumeration& Enumeration, FJsonUETypeWriter& Writer)
{
	// TODO: how to access this
}

#endif

void WriteFFieldClass(const FFieldClass& FieldClass, FJsonUETypeWriter& Writer);

void WriteFField(const FField* Field, FJsonUETypeWriter& Writer)
{	
	FArchiveMD5Generator MD5Archive;
	const_cast<FField*>(Field)->Serialize(MD5Archive);

	// this is a fallback to just make sure that we catch all differences
	Writer.WriteValue(TEXT("MD5"), LexToString(MD5Archive.GetMD5()));

	Writer.WriteValue(TEXT("Name"), Field->GetName());
	Writer.WriteValue(TEXT("ClassName"), Field->GetClass()->GetName());

	if (const FProperty* Property = CastField<const FProperty>(Field))
	{
		Writer.WriteValue(TEXT("ArrayDim"), Property->ArrayDim);
		Writer.WriteValue(TEXT("ElementSize"), Property->GetElementSize());
		Writer.WriteValue(TEXT("PropertyFlags"), FindUsedBits(Property->PropertyFlags));
		Writer.WriteValue(TEXT("RepIndex"), Property->RepIndex);
		Writer.WriteValue(TEXT("RepNotifyFunc"), Property->RepNotifyFunc.ToString());
		Writer.WriteValue(TEXT("Offset_Internal"), Property->GetOffset_ForDebug());

		if (const FBoolProperty* BoolProperty = CastField<const FBoolProperty>(Property))
		{
			Writer.WriteValue(TEXT("FieldSize"), BoolProperty->GetBoolFieldSize());
			Writer.WriteValue(TEXT("FieldMask"), BoolProperty->GetFieldMask());
			Writer.WriteValue(TEXT("ByteOffset"), BoolProperty->GetByteOffset());
			Writer.WriteValue(TEXT("ByteMask"), BoolProperty->GetByteMask());
		}
		else if (const FEnumProperty* EnumProperty = CastField<const FEnumProperty>(Property))
		{
			if (const FNumericProperty* UnderlyingProperty = EnumProperty->GetUnderlyingProperty())
			{
				Writer.WriteObjectStart(TEXT("UnderlyingProperty"));
				WriteFField(UnderlyingProperty, Writer);
				Writer.WriteObjectEnd();
			}

			if (const UEnum* Enum = EnumProperty->GetEnum())
			{
				Writer.WriteValue(TEXT("Enum"), Enum->GetPathName());
			}
		}
		else if (const FObjectPropertyBase* ObjectPropertyBase = CastField<const FObjectPropertyBase>(Property))
		{
			if (ObjectPropertyBase->PropertyClass)
			{
				Writer.WriteValue(TEXT("PropertyClass"), ObjectPropertyBase->PropertyClass->GetName());
			}

			if (const FClassProperty* ClassProperty = CastField<const FClassProperty>(ObjectPropertyBase))
			{
				if (ClassProperty->MetaClass)
				{
					Writer.WriteValue(TEXT("MetaClass"), ClassProperty->MetaClass->GetName());
				}
			}
			else if (const FSoftClassProperty* SoftClassProperty = CastField<const FSoftClassProperty>(ObjectPropertyBase))
			{
				if (SoftClassProperty->MetaClass)
				{
					Writer.WriteValue(TEXT("MetaClass"), SoftClassProperty->MetaClass->GetName());
				}				
			}
		}
		else if (const FStructProperty* StructProperty = CastField<const FStructProperty>(Property))
		{
			if (StructProperty->Struct)
			{
				Writer.WriteValue(TEXT("Struct"), StructProperty->Struct->GetPathName());
			}
		}
		else if (const FMulticastDelegateProperty* MulticastDelegateProperty = CastField<const FMulticastDelegateProperty>(Property))
		{
			if (MulticastDelegateProperty->SignatureFunction)
			{
				Writer.WriteValue(TEXT("SignatureFunction"), MulticastDelegateProperty->SignatureFunction->GetPathName());
			}
		}
		else if (const FByteProperty* ByteProperty = CastField<const FByteProperty>(Property))
		{
			if (ByteProperty->Enum)
			{
				Writer.WriteValue(TEXT("Enum"), ByteProperty->Enum.GetPathName());
			}
		}
		else if (const FMapProperty* MapProperty = CastField<const FMapProperty>(Property))
		{
			if (MapProperty->KeyProp)
			{
				Writer.WriteObjectStart(TEXT("KeyProp"));
				WriteFField(MapProperty->KeyProp, Writer);
				Writer.WriteObjectEnd();
			}

			if (MapProperty->ValueProp)
			{
				Writer.WriteObjectStart(TEXT("ValueProp"));
				WriteFField(MapProperty->ValueProp, Writer);
				Writer.WriteObjectEnd();				
			}

			Writer.WriteValue(TEXT("MapFlags"), FindUsedBits(MapProperty->MapFlags));

			Writer.WriteObjectStart(TEXT("MapLayout"));
			WriteFScriptMapLayout(MapProperty->MapLayout, Writer);
			Writer.WriteObjectEnd();
		}
		else if (const FSetProperty* SetProperty = CastField<const FSetProperty>(Property))
		{
			if (SetProperty->ElementProp)
			{
				Writer.WriteObjectStart(TEXT("ElementProp"));
				WriteFField(SetProperty->ElementProp, Writer);
				Writer.WriteObjectEnd();
			}

			Writer.WriteObjectStart(TEXT("SetLayout"));
			WriteFScriptSetLayout(SetProperty->SetLayout, Writer);
			Writer.WriteObjectEnd();
		}
		else if (const FDelegateProperty* DelegateProperty = CastField<const FDelegateProperty>(Property))
		{
			if (DelegateProperty->SignatureFunction)
			{
				Writer.WriteValue(TEXT("SignatureFunction"), DelegateProperty->SignatureFunction->GetPathName());
			}
		}
		else if (const FArrayProperty* ArrayProperty = CastField<const FArrayProperty>(Property))
		{
			Writer.WriteValue(TEXT("ArrayFlags"), FindUsedBits(ArrayProperty->ArrayFlags));
			if (ArrayProperty->Inner)
			{
				Writer.WriteObjectStart(TEXT("Inner"));
				WriteFField(ArrayProperty->Inner, Writer);
				Writer.WriteObjectEnd();
			}
		}
		else if (const FOptionalProperty* OptionalProperty = CastField<const FOptionalProperty>(Property))
		{
			Writer.WriteObjectStart(TEXT("ValueProperty"));
			WriteFField(OptionalProperty->GetValueProperty(), Writer);
			Writer.WriteObjectEnd();
		}
		else if (const FInterfaceProperty* InterfaceProperty = CastField<const FInterfaceProperty>(Property))
		{
			if (InterfaceProperty->InterfaceClass)
			{
				Writer.WriteValue(TEXT("InterfaceClass"), InterfaceProperty->InterfaceClass->GetPathName());
			}
		}
	}
}


void WriteFFieldClass(const FFieldClass& FieldClass, FJsonUETypeWriter& Writer)
{
	Writer.WriteValue(TEXT("Name"), FieldClass.GetName());
	Writer.WriteValue(TEXT("Id"), FieldClass.GetId());
	Writer.WriteValue(TEXT("CastFlags"), FindUsedBits(FieldClass.GetCastFlags()));

	// just a way to bypass private restrictions
	EClassFlags ClassFlags = CLASS_None;
	for (int32 Bit = 0; Bit < 32; ++Bit)
	{
		EClassFlags TestFlag = EClassFlags(1<<Bit);
		if (FieldClass.HasAnyClassFlags(TestFlag))
		{
			ClassFlags |= TestFlag;
		}
	}

	// TODO: right now ClassFlags is always 0 in all of these types, so either this file logic can't find them
	// or there are elements that are unused here
	Writer.WriteValue(TEXT("ClassFlags"), FindUsedBits(ClassFlags));

	if (const FFieldClass* SuperClass = FieldClass.GetSuperClass())
	{
		Writer.WriteValue(TEXT("SuperClass"), SuperClass->GetName());
	}

	if (ClassFlags != CLASS_None)
	{
		Writer.WriteObjectStart(TEXT("DefaultObject"));
		WriteFField(const_cast<FFieldClass&>(FieldClass).GetDefaultObject(), Writer);
		Writer.WriteObjectEnd();		
	}
}

void WriteUField(const UField* Field, FJsonUETypeWriter& Writer)
{
	FArchiveMD5Generator MD5Archive;
	const_cast<UField*>(Field)->Serialize(MD5Archive);

	// this is a fallback to just make sure that we catch all differences
	Writer.WriteValue(TEXT("MD5"), LexToString(MD5Archive.GetMD5()));


	Writer.WriteObjectStart(TEXT("UObject"));
	Writer.WriteValue(TEXT("Name"), Field->GetName());
	Writer.WriteValue(TEXT("Flags"), LexToString(Field->GetFlags()));
	Writer.WriteObjectEnd();

#if WITH_METADATA
	Writer.WriteObjectStart(TEXT("UField"));
	if (const TMap<FName, FString>* MetaData = FMetaData::GetMapForObject(Field))
	{
		if (!MetaData->IsEmpty())
		{
			Writer.WriteObjectStart(TEXT("MetaData"));

			TArray<FName> Keys;
			MetaData->GetKeys(Keys);
			Keys.Sort([](const FName& A, const FName& B){
				return A.Compare(B) < 0;
			});

			for (const FName& Key : Keys)
			{
				Writer.WriteValue(*Key.ToString(), *MetaData->Find(Key));
			}

			Writer.WriteObjectEnd();
		}
	}
	Writer.WriteObjectEnd();
#endif

	if (const UEnum* Enum = Cast<const UEnum>(Field))
	{
		Writer.WriteObjectStart(TEXT("UEnum"));

		Writer.WriteValue(TEXT("CppType"), Enum->CppType);
		Writer.WriteValue(TEXT("CppForm"), int32(Enum->GetCppForm()));
		Writer.WriteValue(TEXT("EnumFlags_Flags"), Enum->HasAnyEnumFlags(EEnumFlags::Flags));
		Writer.WriteValue(TEXT("EnumFlags_NewerVersionExists"), Enum->HasAnyEnumFlags(EEnumFlags::NewerVersionExists));

		TArray<TPair<FName, int32>> EnumValues;
		for (int32 Index = 0; Index < Enum->NumEnums(); ++Index)
		{
			const FName Name = Enum->GetNameByIndex(Index);
			if (Name == NAME_None)
			{
				continue;
			}

			const int64 Value = Enum->GetValueByIndex(Index);

			EnumValues.Add({Name, Value});
		}

		EnumValues.Sort([](const TPair<FName, int32>& A, const TPair<FName, int32>& B)
		{
			return A.Value < B.Value;
		});

		Writer.WriteObjectStart(TEXT("Names"));
		for (const TPair<FName, int32>& Pair : EnumValues)
		{
			Writer.WriteObjectStart(Pair.Key.ToString());
			Writer.WriteValue(TEXT("Value"), Pair.Value);
			// this exists to indirectly ensure that the display name fn was set correctly
			Writer.WriteValue(TEXT("DisplayName"), Enum->GetDisplayNameTextByValue(Pair.Value).ToString());
			Writer.WriteObjectEnd();
		}
		Writer.WriteObjectEnd();

		if (const UVerseEnum* VerseEnum = Cast<const UVerseEnum>(Enum))
		{
			Writer.WriteObjectStart(TEXT("UVerseEnum"));
			Writer.WriteValue(TEXT("VerseEnumFlags"), UEnumFlagsToString(VerseEnum->VerseEnumFlags));
			Writer.WriteValue(TEXT("QualifiedName"), VerseEnum->QualifiedName);

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
			WriteVerseVEnumeration(TEXT("Enumeration"), *VerseEnum->Enumeration, Writer);
#endif
			Writer.WriteObjectEnd();
		}

		Writer.WriteObjectEnd();
	}
	else if (const UStruct* Struct = Cast<const UStruct>(Field))
	{
		Writer.WriteObjectStart(TEXT("UStruct"));

		if (UStruct* SuperStruct = Struct->GetSuperStruct())
		{
			Writer.WriteValue(TEXT("SuperStruct"), SuperStruct->GetName());
		}

		{
			TArray<FString> Children;
			for (const UField* Current = Struct->Children.Get(); Current; Current = Current->Next)
			{
				Children.Add(Current->GetPathName());
			}
			Children.Sort();

			Writer.WriteValue(TEXT("Children"), Children);
		}

		{
			TArray<const FField*> ChildProperties;
			for (const FField* Current = Struct->ChildProperties; Current; Current = Current->Next)
			{
				ChildProperties.Add(Current);
			}

			WriteSortedObjectHelper(
				TEXT("ChildProperties"),
				ChildProperties,
				Writer,
				[](const FField* Field)
				{
					return Field->GetFullName();
				},
				[&Writer](const FField* Field)
				{
					WriteFField(Field, Writer);
				}
			);
		}

		Writer.WriteValue(TEXT("PropertiesSize"), Struct->PropertiesSize);
		Writer.WriteValue(TEXT("MinAlignment"), Struct->MinAlignment);

	#if DO_CHECK
		// TODO: this isn't exported, is it needed?
		//Writer.WriteValue(TEXT("IsPropertyChainReady"), Struct->DebugIsPropertyChainReady());
	#endif

		TArray<FString> PathNames;
		for (const FProperty* Current = Struct->PropertyLink; Current; Current = Current->PropertyLinkNext)
		{
			PathNames.Add(Current->GetFullName());
		}
		PathNames.Sort();
		Writer.WriteValue(TEXT("PropertyLinks"), PathNames);

		PathNames.Reset();
		for (const FProperty* Current = Struct->RefLink; Current; Current = Current->NextRef)
		{
			PathNames.Add(Current->GetFullName());
		}
		PathNames.Sort();
		Writer.WriteValue(TEXT("RefLinks"), PathNames);

		PathNames.Reset();
		for (const FProperty* Current = Struct->DestructorLink; Current; Current = Current->DestructorLinkNext)
		{
			PathNames.Add(Current->GetFullName());
		}
		PathNames.Sort();
		Writer.WriteValue(TEXT("DestructorLinks"), PathNames);

		PathNames.Reset();
		for (const FProperty* Current = Struct->PostConstructLink; Current; Current = Current->PostConstructLinkNext)
		{
			PathNames.Add(Current->GetFullName());
		}
		PathNames.Sort();
		Writer.WriteValue(TEXT("PostConstructLinks"), PathNames);


		PathNames.Reset();
		for (const TObjectPtr<UObject>& Object : Struct->ScriptAndPropertyObjectReferences)
		{
			if (Object)
			{
				PathNames.Add(Object->GetPathName());
			}
			else
			{
				PathNames.Add(TEXT("(null)"));
			}
		}
		PathNames.Sort();
		Writer.WriteValue(TEXT("ScriptAndPropertyObjectReferences"), PathNames);

		if (Struct->UnresolvedScriptProperties)
		{
			Writer.WriteArrayStart(TEXT("UnresolvedScriptProperties"));
			for (const TPair<TFieldPath<FField>, int32>& UnresolvedScriptProperty : *Struct->UnresolvedScriptProperties)
			{
				Writer.WriteObjectStart();
				Writer.WriteValue(TEXT("Path"), UnresolvedScriptProperty.Key.ToString());
				Writer.WriteValue(TEXT("Value"), UnresolvedScriptProperty.Value);
				Writer.WriteObjectEnd();
			}
			Writer.WriteArrayEnd();
		}

	#if WITH_EDITORONLY_DATA
		PathNames.Reset();
		for (const TObjectPtr<UPropertyWrapper>& PropertyWrapper : Struct->PropertyWrappers)
		{
			PathNames.Add(PropertyWrapper->GetPathName());
		}
		PathNames.Sort();
		Writer.WriteValue(TEXT("PropertyWrappers"), PathNames);
		Writer.WriteValue(TEXT("TotalFieldCount"), Struct->TotalFieldCount);
		Writer.WriteValue(TEXT("HasAssetRegistrySearchableProperties"), Struct->HasAssetRegistrySearchableProperties());
	#endif

		Writer.WriteObjectEnd(); // UStruct

		if (const UScriptStruct* ScriptStruct = Cast<const UScriptStruct>(Struct))
		{
			Writer.WriteObjectStart(TEXT("UScriptStruct"));

			Writer.WriteValue(TEXT("StructFlags"), FindUsedBits(ScriptStruct->StructFlags));
			Writer.WriteValue(TEXT("CustomGuid"), ScriptStruct->GetCustomGuid().ToString());
			Writer.WriteValue(TEXT("StructCppName"), ScriptStruct->GetStructCPPName());

			if (const UScriptStruct::ICppStructOps* StructOps = ScriptStruct->GetCppStructOps())
			{
				Writer.WriteObjectStart(TEXT("StructOps"));
				Writer.WriteValue(TEXT("Size"), StructOps->GetSize());
				Writer.WriteValue(TEXT("Alignment"), StructOps->GetAlignment());

				const UScriptStruct::ICppStructOps::FCapabilities Capabilities = StructOps->GetCapabilities();
				Writer.WriteObjectStart(TEXT("Capabilities"));
				Writer.WriteValue(TEXT("ComputedPropertyFlags"), FindUsedBits(Capabilities.ComputedPropertyFlags));
				Writer.WriteValue(TEXT("HasSerializerObjectReferences"), LexToString(Capabilities.HasSerializerObjectReferences));

				Writer.WriteValue(TEXT("HasNoopConstructor"), Capabilities.HasNoopConstructor);
				Writer.WriteValue(TEXT("HasZeroConstructor"), Capabilities.HasZeroConstructor);
				Writer.WriteValue(TEXT("HasDestructor"), Capabilities.HasDestructor);
				Writer.WriteValue(TEXT("HasSerializer"), Capabilities.HasSerializer);
				Writer.WriteValue(TEXT("HasStructuredSerializer"), Capabilities.HasStructuredSerializer);
				Writer.WriteValue(TEXT("HasPostSerialize"), Capabilities.HasPostSerialize);
				Writer.WriteValue(TEXT("HasNetSerializer"), Capabilities.HasNetSerializer);
				Writer.WriteValue(TEXT("HasNetSharedSerialization"), Capabilities.HasNetSharedSerialization);
				Writer.WriteValue(TEXT("HasNetDeltaSerializer"), Capabilities.HasNetDeltaSerializer);
				Writer.WriteValue(TEXT("HasPostScriptConstruct"), Capabilities.HasPostScriptConstruct);
				Writer.WriteValue(TEXT("IsPlainOldData"), Capabilities.IsPlainOldData);
				Writer.WriteValue(TEXT("IsUECoreType"), Capabilities.IsUECoreType);
				Writer.WriteValue(TEXT("IsUECoreVariant"), Capabilities.IsUECoreVariant);
				Writer.WriteValue(TEXT("HasCopy"), Capabilities.HasCopy);
				Writer.WriteValue(TEXT("HasIdentical"), Capabilities.HasIdentical);
				Writer.WriteValue(TEXT("HasExportTextItem"), Capabilities.HasExportTextItem);
				Writer.WriteValue(TEXT("HasImportTextItem"), Capabilities.HasImportTextItem);
				Writer.WriteValue(TEXT("HasAddStructReferencedObjects"), Capabilities.HasAddStructReferencedObjects);
				Writer.WriteValue(TEXT("HasSerializeFromMismatchedTag"), Capabilities.HasSerializeFromMismatchedTag);
				Writer.WriteValue(TEXT("HasStructuredSerializeFromMismatchedTag"), Capabilities.HasStructuredSerializeFromMismatchedTag);
				Writer.WriteValue(TEXT("HasGetTypeHash"), Capabilities.HasGetTypeHash);
				Writer.WriteValue(TEXT("HasIntrusiveUnsetOptionalState"), Capabilities.HasIntrusiveUnsetOptionalState);
				Writer.WriteValue(TEXT("IsAbstract"), Capabilities.IsAbstract);
				Writer.WriteValue(TEXT("HasFindInnerPropertyInstance"), Capabilities.HasFindInnerPropertyInstance);
				Writer.WriteValue(TEXT("ClearOnFinishDestroy"), Capabilities.ClearOnFinishDestroy);
#if WITH_EDITOR
				Writer.WriteValue(TEXT("HasCanEditChange"), Capabilities.HasCanEditChange);
#endif
				Writer.WriteValue(TEXT("HasVisitor"), Capabilities.HasVisitor);

				Writer.WriteObjectEnd();

				Writer.WriteObjectEnd();
			}

			if (const UVerseStruct* VerseStruct = Cast<const UVerseStruct>(Struct))
			{
				Writer.WriteObjectStart(TEXT("VerseStruct"));

				Writer.WriteValue(TEXT("VerseClassFlags"), FindUsedBits(VerseStruct->VerseClassFlags));
				Writer.WriteValue(TEXT("QualifiedName"), VerseStruct->QualifiedName);

				if (VerseStruct->InitFunction)
				{
					Writer.WriteValue(TEXT("InitFunction"), VerseStruct->InitFunction->GetPathName());
				}

				if (VerseStruct->ModuleClass)
				{
					Writer.WriteValue(TEXT("ModuleClass"), VerseStruct->ModuleClass->GetPathName());
				}

				Writer.WriteValue(TEXT("Guid"), VerseStruct->Guid.ToString());

				if (VerseStruct->FactoryFunction)
				{
					Writer.WriteValue(TEXT("FactoryFunction"), VerseStruct->FactoryFunction->GetPathName());
				}

				if (VerseStruct->OverrideFactoryFunction)
				{
					Writer.WriteValue(TEXT("OverrideFactoryFunction"), VerseStruct->OverrideFactoryFunction->GetPathName());
				}

				Writer.WriteValue(TEXT("ConstructorEffects"), UEnumFlagsToString(VerseStruct->ConstructorEffects));

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
				WriteVerseVClass(TEXT("Class"), VerseStruct->Class, Writer);
				WriteVerseVShape(TEXT("Shape"), VerseStruct->Shape, Writer);
#endif				

				Writer.WriteObjectEnd();
			}

			Writer.WriteObjectEnd();
		}
		else if (const UFunction* Function = Cast<const UFunction>(Struct))
		{
			Writer.WriteObjectStart(TEXT("UFunction"));

			Writer.WriteValue(TEXT("FunctionFlags"), FindUsedBits(Function->FunctionFlags));
			Writer.WriteValue(TEXT("NumParms"), Function->NumParms);
			Writer.WriteValue(TEXT("ParmsSize"), Function->ParmsSize);
			Writer.WriteValue(TEXT("ReturnValueOffset"), Function->ReturnValueOffset);
			Writer.WriteValue(TEXT("RPCId"), Function->RPCId);
			Writer.WriteValue(TEXT("RPCResponseId"), Function->RPCResponseId);

			if (Function->FirstPropertyToInit)
			{
				Writer.WriteValue(TEXT("FirstPropertyToInit"), Function->GetFullName());
			}


#if UE_BLUEPRINT_EVENTGRAPH_FASTCALLS
			if (Function->EventGraphFunction)
			{
				Writer.WriteValue(TEXT("EventGraphFunction"), Function->EventGraphFunction->GetPathName());
			}
			Writer.WriteValue(TEXT("EventGraphCallOffset"), Function->EventGraphCallOffset);
#endif			

#if WITH_LIVE_CODING
			if (Function->SingletonPtr)
			{
				// just a guess that this is what is intended
				Writer.WriteValue(TEXT("SingletonPtrEqualsThis"), *Function->SingletonPtr == Function);
			}
#endif
			ANSICHAR NativeFuncName[512] = {0};
			if (Function->GetNativeFunc() == nullptr)
			{
				Writer.WriteValue(TEXT("NativeFunc"), TEXT("nullptr"));
			}
			else
			{
				if (FPlatformStackWalk::ProgramCounterToHumanReadableString( 0, uint64(Function->GetNativeFunc()), NativeFuncName, 512, nullptr ))
				{
					FString NativeFuncString(ANSI_TO_TCHAR(NativeFuncName));
					TArray<FStringView> NamePartStrings;
					UE::String::ParseTokens(NativeFuncString, TEXT(" "), [&NamePartStrings](FStringView Token) { NamePartStrings.Add(Token); });

					if (NamePartStrings.Num() > 1)
					{
						// this is stripping off the address prefix
						Writer.WriteValue(TEXT("NativeFunc"), NamePartStrings[1]);
					}
					else
					{
						Writer.WriteValue(TEXT("NativeFunc"), NativeFuncString);
					}
				}
				else
				{
					Writer.WriteValue(TEXT("NativeFunc"), TEXT("Unknown Set Function"));
				}
			}
		
			Writer.WriteObjectEnd();

			if (const USparseDelegateFunction* SparseDelegateFunction = Cast<const USparseDelegateFunction>(Function))
			{
				Writer.WriteObjectStart(TEXT("USparseDelegateFunction"));
				Writer.WriteValue(TEXT("OwningClassName"), SparseDelegateFunction->OwningClassName.ToString());
				Writer.WriteValue(TEXT("DelegateName"), SparseDelegateFunction->DelegateName.ToString());
				Writer.WriteObjectEnd();
			}

			if (const UVerseFunction* VerseFunction = Cast<const UVerseFunction>(Function))
			{
				Writer.WriteObjectStart(TEXT("UVerseFunction"));
				Writer.WriteValue(TEXT("AlternateName"), VerseFunction->AlternateName.ToString());
				Writer.WriteValue(TEXT("VerseFunctionFlags"), UEnumFlagsToString(VerseFunction->VerseFunctionFlags));

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
				WriteVerseVFunction(TEXT("Callee"), *VerseFunction->Callee, Writer);
#endif

				Writer.WriteObjectEnd();
			}
		}
		else if (const UClass* Class = Cast<const UClass>(Struct))
		{
			Writer.WriteObjectStart(TEXT("UClass"));

			Writer.WriteValue(TEXT("bCooked"), Class->bCooked);
			Writer.WriteValue(TEXT("bLayoutChanging"), Class->bLayoutChanging);
			Writer.WriteValue(TEXT("ClassFlags"), FindUsedBits(Class->ClassFlags));
			Writer.WriteValue(TEXT("ClassCastFlags"), FindUsedBits(Class->ClassCastFlags));

			if (Class->ClassWithin)
			{
				Writer.WriteValue(TEXT("ClassWithin"), Class->ClassWithin->GetPathName());
			}

			Writer.WriteValue(TEXT("ClassConfigName"), Class->ClassConfigName.ToString());

			WriteSortedObjectHelper(
				TEXT("ClassReps"),
				Class->ClassReps,
				Writer,
				[](const FRepRecord& Record)
				{
					return Record.Property->GetFullName();
				},
				[&Writer](const FRepRecord& Record)
				{
					Writer.WriteValue(TEXT("Index"), Record.Index);
				}
			);

			TArray<FString> NetFields;
			for (const UField* NetField : Class->NetFields)
			{
				NetFields.Add(NetField->GetPathName());
			}

			NetFields.Sort();
			Writer.WriteValue(TEXT("NetFields"), NetFields);

			if (const UScriptStruct* SparseClassDataStruct = Class->GetSparseClassDataStruct())
			{
				Writer.WriteValue(TEXT("SparseClassDataStruct"), SparseClassDataStruct->GetPathName());
			}

#if WITH_EDITOR
			if (const ICppClassTypeInfo* CppTypeInfo = Class->GetCppTypeInfo())
			{
				Writer.WriteObjectStart(TEXT("CppTypeInfo"));
				Writer.WriteValue(TEXT("IsAbstract"), CppTypeInfo->IsAbstract());
				Writer.WriteObjectEnd();
			}
#endif
			if (const UVerseClass* VerseClass = Cast<const UVerseClass>(Class))
			{
				Writer.WriteObjectStart(TEXT("UVerseClass"));

				Writer.WriteValue(TEXT("SolClassFlags"), FindUsedBits(VerseClass->SolClassFlags));

				Writer.WriteArrayStart(TEXT("TaskClasses"));
				for (const TObjectPtr<UVerseClass>& TaskClass : VerseClass->TaskClasses)
				{
					if (TaskClass)
					{
						Writer.WriteValue(TaskClass->GetPathName());
					}
					else
					{
						Writer.WriteValue(TEXT("null"));
					}
				}
				Writer.WriteArrayEnd();

				if (VerseClass->InitInstanceFunction)
				{
					Writer.WriteValue(TEXT("InitInstanceFunction"), VerseClass->InitInstanceFunction->GetPathName());
				}

				Writer.WriteArrayStart(TEXT("PersistentVars"));
				for (const FVersePersistentVar& PersistentVar : VerseClass->PersistentVars)
				{
					Writer.WriteObjectStart();

					Writer.WriteValue(TEXT("Path"), PersistentVar.Path);
					Writer.WriteValue(TEXT("RelativePath"), PersistentVar.RelativePath);
					Writer.WriteValue(TEXT("Property"), PersistentVar.Property.ToString());

					Writer.WriteObjectEnd();
				}
				Writer.WriteArrayEnd();

				Writer.WriteArrayStart(TEXT("SessionVars"));
				for (const FVerseSessionVar& SessionVar : VerseClass->SessionVars)
				{
					Writer.WriteValue(SessionVar.Property.ToString());
				}
				Writer.WriteArrayEnd();

				Writer.WriteObjectStart(TEXT("VarAccessors"));
				for (const TPair<FName, FVerseClassVarAccessors>& Pair : VerseClass->VarAccessors)
				{
					Writer.WriteObjectStart(Pair.Key.ToString());

					auto WriteFVerseClassVarAccessorKeyValue = [&Writer](const TPair<int, FVerseClassVarAccessor>& AccessorPair)
					{
						Writer.WriteObjectStart(FString::Printf(TEXT("%d"), AccessorPair.Key));

						if (AccessorPair.Value.Func)
						{
							Writer.WriteValue(TEXT("Func"), AccessorPair.Value.Func->GetPathName());
						}
						Writer.WriteValue(TEXT("bIsInstanceMember"), AccessorPair.Value.bIsInstanceMember);
						Writer.WriteValue(TEXT("bIsFallible"), AccessorPair.Value.bIsFallible);

						Writer.WriteObjectEnd();
					};

					Writer.WriteObjectStart(TEXT("Getters"));
					for (const TPair<int, FVerseClassVarAccessor>& Getter : Pair.Value.Getters)
					{
						WriteFVerseClassVarAccessorKeyValue(Getter);
					}
					Writer.WriteObjectEnd();

					Writer.WriteObjectStart(TEXT("Setters"));
					for (const TPair<int, FVerseClassVarAccessor>& Setter : Pair.Value.Setters)
					{
						WriteFVerseClassVarAccessorKeyValue(Setter);
					}
					Writer.WriteObjectEnd();

					Writer.WriteObjectEnd();
				}
				Writer.WriteObjectEnd();

				Writer.WriteValue(TEXT("ConstructorEffects"), UEnumFlagsToString(VerseClass->ConstructorEffects));
				Writer.WriteValue(TEXT("MangledPackageVersePath"), VerseClass->MangledPackageVersePath.ToString());
				Writer.WriteValue(TEXT("PackageRelativeVersePath"), VerseClass->PackageRelativeVersePath);

				auto WriteNameMap = [&Writer](const TCHAR* ObjectName, const TMap<FName, FName>& NameMap)
				{
					if (NameMap.IsEmpty())
					{
						return;
					}

					Writer.WriteObjectStart(ObjectName);
					for (const TPair<FName, FName>& Pair : NameMap)
					{
						Writer.WriteValue(Pair.Key.ToString(), Pair.Value.ToString());
					}
					Writer.WriteObjectEnd();
				};

				WriteNameMap(TEXT("DisplayNameToUENameFunctionMap"), VerseClass->DisplayNameToUENameFunctionMap);

				Writer.WriteArrayStart(TEXT("DirectInterfaces"));
				for (const TObjectPtr<UVerseClass>& DirectInterface : VerseClass->DirectInterfaces)
				{
					if (DirectInterface)
					{
						Writer.WriteValue(DirectInterface->GetPathName());
					}
					else
					{
						Writer.WriteValue(TEXT("null"));
					}
				}
				Writer.WriteArrayEnd();

				if (!VerseClass->PropertiesWrittenByInitCDO.IsEmpty())
				{
					Writer.WriteArrayStart(TEXT("PropertiesWrittenByInitCDO"));
					for (const TFieldPath<FProperty>& Path : VerseClass->PropertiesWrittenByInitCDO)
					{
						Writer.WriteValue(Path.ToString());
					}
					Writer.WriteArrayEnd();
				}

				WriteNameMap(TEXT("FunctionMangledNames"), VerseClass->FunctionMangledNames);

				if (!VerseClass->PredictsFunctionNames.IsEmpty())
				{
					Writer.WriteArrayStart(TEXT("PredictsFunctionNames"));
					for (FName Name : VerseClass->PredictsFunctionNames)
					{
						Writer.WriteValue(Name.ToString());
					}
					Writer.WriteArrayEnd();
				}

#if WITH_VERSE_COMPILER && WITH_EDITORONLY_DATA
				Writer.WriteValue(TEXT("PreviousPathName"), VerseClass->PreviousPathName);
#endif
#if WITH_VERSE_VM || defined(__INTELLISENSE__)				
				WriteVerseVClass(TEXT("Class"), VerseClass->Class, Writer);
				WriteVerseVShape(TEXT("Shape"), VerseClass->Shape, Writer);
#endif
				Writer.WriteObjectEnd();
			}

			Writer.WriteObjectEnd();
		}
	}
}

// if bMonolithic is true, then "Filename" is the base folder name
void DumpUETypeInformation(const FString& Filename, bool bMonolithic, bool bOnlyCompiledIn)
{
	TMap<UPackage*, TArray<const UField*>> FieldsByPackage;
	for (TObjectIterator<UField> It; It; ++It)
	{		
		FieldsByPackage.FindOrAdd(It->GetPackage()).Add(*It);
	}

	// now write them in a per-package json

	FString MonolithicOutputString;

	{
		FJsonUETypeWriter MonolithicWriter(&MonolithicOutputString);

		MonolithicWriter.WriteObjectStart();

		MonolithicWriter.WriteObjectStart(TEXT("UFields"));

		for (TPair<UPackage*, TArray<const UField*>>& Pair : FieldsByPackage)
		{
			if (bOnlyCompiledIn && !Pair.Key->HasAnyPackageFlags(PKG_CompiledIn))
			{
				continue;
			}

			FString FileOutputString;
			FJsonUETypeWriter PackageWriter(&FileOutputString);

			FJsonUETypeWriter& Writer = bMonolithic ? MonolithicWriter : PackageWriter;

			if (bMonolithic)
			{
				Writer.WriteObjectStart(Pair.Key->GetName());
			}
			else
			{
				Writer.WriteObjectStart();
			}

			TArray<const UField*>& AllFields = Pair.Value;

			AllFields.Sort([](const UField& A, const UField& B){
				return A.GetPathName().Compare(B.GetPathName()) < 0;
			});

			for (const UField* Field : AllFields)
			{
				Writer.WriteObjectStart(*Field->GetPathName());
				WriteUField(Field, Writer);
				Writer.WriteObjectEnd();
			}

			Writer.WriteObjectEnd();

			if (!bMonolithic)
			{
				PackageWriter.Close();
				FString DumpFilename = FPaths::ProjectSavedDir() / Filename / FString::Printf(TEXT("%s.json"), *Pair.Key->GetPathName());
				FFileHelper::SaveStringToFile(FileOutputString, *DumpFilename, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);
			}
		}

		MonolithicWriter.WriteObjectEnd(); // UFields

		// dump all field classes to a separate file/chunk

		{
			FString FileOutputString;
			FJsonUETypeWriter SplitWriter(&FileOutputString);

			FJsonUETypeWriter& Writer = bMonolithic ? MonolithicWriter : SplitWriter;

			if (bMonolithic)
			{
				Writer.WriteObjectStart(TEXT("FFieldClasses"));
			}
			else
			{
				Writer.WriteObjectStart();
			}

			TArray<FFieldClass*> FieldClasses = FFieldClass::GetAllFieldClasses();
			FieldClasses.Sort([](FFieldClass& A, FFieldClass& B)
			{
				return A.GetName().Compare(B.GetName()) < 0;
			});

			for (FFieldClass* FieldClass : FieldClasses)
			{
				Writer.WriteObjectStart(FieldClass->GetName());
				WriteFFieldClass(*FieldClass, Writer);
				Writer.WriteObjectEnd();
			}

			Writer.WriteObjectEnd();

			if (!bMonolithic)
			{
				SplitWriter.Close();
				FString DumpFilename = FPaths::ProjectSavedDir() / Filename / TEXT("FieldClasses.json");
				FFileHelper::SaveStringToFile(FileOutputString, *DumpFilename, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);
			}			
		}

		MonolithicWriter.WriteObjectEnd(); // root object
		MonolithicWriter.Close();
	}

	if (bMonolithic)
	{
		FString DumpFilename = FPaths::ProjectSavedDir() / Filename;
		FFileHelper::SaveStringToFile(MonolithicOutputString, *DumpFilename, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);		
	}
}

FAutoConsoleCommand GDumpUETypeInformation(
	TEXT("JsonUtilities.DumpUETypeInformation"),
	TEXT("Dumps all of the type information into a monolithic json file"),
	FConsoleCommandWithArgsDelegate::CreateLambda( [](const TArray<FString>& Args )
	{
		if (Args.Num() == 0)
		{
			return;
		}

		bool bOnlyCompiledIn = false;
		if (Args.Num() > 1)
		{
			LexFromString(bOnlyCompiledIn, *Args[1]);
		}

		DumpUETypeInformation(Args[0], true, bOnlyCompiledIn);
	})
);

FAutoConsoleCommand GDumpSplitUETypeInformation(
	TEXT("JsonUtilities.DumpSplitUETypeInformation"),
	TEXT("Dumps all of the type information into a folder"),
	FConsoleCommandWithArgsDelegate::CreateLambda( [](const TArray<FString>& Args )
	{
		if (Args.Num() == 0)
		{
			return;
		}

		bool bOnlyCompiledIn = false;
		if (Args.Num() > 1)
		{
			LexFromString(bOnlyCompiledIn, *Args[1]);
		}

		DumpUETypeInformation(Args[0], false, bOnlyCompiledIn);
	})
);

} // namespace UE::JsonUtilities

#endif // !UE_BUILD_SHIPPING
