// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonUtilitiesModularFeature.h"

/*static*/ const FName IJsonUtilitiesModularFeature::Name(TEXT("JsonUtilitiesModularFeature"));

IJsonUtilitiesModularFeature::IJsonUtilitiesModularFeature()
{
}

/*static*/ IJsonUtilitiesModularFeature* IJsonUtilitiesModularFeature::Get()
{
	// See all NOTE_JSON_UTILITIES_SCHEMA_GENERATOR_METADATA_MODULAR_FEATURE.
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	return (ModularFeatures.IsModularFeatureAvailable(Name) ?
		&ModularFeatures.GetModularFeature<IJsonUtilitiesModularFeature>(Name) :
		nullptr);
}
