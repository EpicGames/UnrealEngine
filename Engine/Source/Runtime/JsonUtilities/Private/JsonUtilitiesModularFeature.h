// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"

#include "JsonSchema/JsonSchemaPropertyFilter.h"
#include "JsonSchema/JsonSchemaEditorMetadata.h"

/**
 * Base class for Modular Feature - Editor functionality that the Runtime module will call when in Editor, to collect metadata.
 * See Editor module for derivation and registration.
 */
class IJsonUtilitiesModularFeature : public IModularFeature
{
public:
	
	JSONUTILITIES_API static const FName Name;

	JSONUTILITIES_API IJsonUtilitiesModularFeature();
	virtual ~IJsonUtilitiesModularFeature() = default;
	
	JSONUTILITIES_API static IJsonUtilitiesModularFeature* Get();

	
	virtual FJsonSchemaEditorMetadata FPropertyToJsonSchemaMetadata(TNotNull<const FProperty*> Property,
		const FJsonSchemaPropertyFilter& PropertyFilter, const void* InstanceMemory) const = 0;

	virtual FJsonSchemaEditorMetadata UStructToJsonSchemaMetadata(TNotNull<const UStruct*> Struct,
		const FJsonSchemaPropertyFilter& PropertyFilter, const void* InstanceMemory) const = 0;
};
