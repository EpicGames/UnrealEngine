// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonSerializer.h"

#include "Templates/UnrealTypeTraits.h"

/**
 * Helpers for creating TSharedPtr<FJsonValue> JSON trees
 *
 * Simple example:
 *
 *	FJsonDomBuilder::FArray InnerArray;
 *	InnerArray.Add(7.f, TEXT("Hello"), true);
 *
 *	FJsonDomBuilder::FObject Object;
 *	Object.Set(TEXT("Array"), InnerArray);
 *	Object.Set(TEXT("Number"), 13.f);
 *
 *	Object.AsJsonValue();
 *
 * produces {"Array": [7., "Hello", true], "Number": 13.}
 */

class FJsonDomBuilder
{
public:
	class FArray;

	class FObject
	{
	public:
		FObject()
			: Object(MakeShared<FJsonObject>())
		{
		}

		FObject(TSharedPtr<FJsonObject> InObject)
			: Object(InObject.ToSharedRef())
		{
		}

		FObject(TSharedRef<FJsonObject> InObject)
			: Object(InObject)
		{
		}

		TSharedRef<FJsonValueObject> AsJsonValue() const
		{
			return MakeShared<FJsonValueObject>(Object);
		}

		TSharedRef<FJsonObject> AsJsonObject() const
		{
			return Object;
		}

		template <template <class> class TPrintPolicy = TPrettyJsonPrintPolicy>
		FString ToString() const
		{
			FString Result;
			auto JsonWriter = TJsonWriterFactory<TCHAR, TPrintPolicy<TCHAR>>::Create(&Result);
			FJsonSerializer::Serialize(Object, JsonWriter);
			return Result;
		}

		template <template <class> class TPrintPolicy = TPrettyJsonPrintPolicy>
		FUtf8String ToUtf8String() const
		{
			FUtf8String Result;
			auto JsonWriter = TJsonWriterFactory<UTF8CHAR, TPrintPolicy<UTF8CHAR>>::Create(&Result);
			FJsonSerializer::Serialize(Object, JsonWriter);
			return Result;
		}

		int Num() const
		{
			return Object->Values.Num();
		}

		FObject& Set(const FString& Key, const FArray& Arr)            { Object->SetField(Key, Arr.AsJsonValue());                            return *this; }
		FObject& Set(const FString& Key, const FObject& Obj)           { Object->SetField(Key, Obj.AsJsonValue());                            return *this; }

		template <class StringType
			UE_REQUIRES(std::is_constructible_v<FString, StringType>)>
		FObject& Set(const FString& Key, const StringType& Str)        { Object->SetField(Key, MakeShared<FJsonValueString>(FString(Str)));   return *this; }

		template <class NumberType
			UE_REQUIRES(!std::is_same_v<NumberType, bool> && (std::is_integral_v<NumberType> || std::is_floating_point_v<NumberType>))>
		FObject& Set(const FString& Key, NumberType Number)            { Object->SetField(Key, MakeShared<FJsonValueNumber>(Number));         return *this; }

		template <class BoolType
			UE_REQUIRES(std::is_same_v<BoolType, bool>)>
		FObject& Set(const FString& Key, BoolType Boolean)             { Object->SetField(Key, MakeShared<FJsonValueBoolean>(Boolean));       return *this; }

		FObject& Set(const FString& Key, TYPE_OF_NULLPTR)              { Object->SetField(Key, MakeShared<FJsonValueNull>());                 return *this; }

		FObject& Set(const FString& Key, TSharedPtr<FJsonValue> Value) { Object->SetField(Key, Value ? Value : MakeShared<FJsonValueNull>()); return *this; }

		void CopyIf(const FJsonObject& Src, TFunctionRef<bool (const FJsonObject::FStringType&, const FJsonValue&)> Pred)
		{
			for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& KV: Src.Values)
			{
				if (ensure(KV.Value) && Pred(KV.Key, *KV.Value))
				{
					Object->SetField(KV.Key, KV.Value);
				}
			}
		}

	private:
		TSharedRef<FJsonObject> Object;
	};

	class FArray
	{
	public:
		TSharedRef<FJsonValueArray> AsJsonValue() const
		{
			return MakeShared<FJsonValueArray>(Array);
		}

		template <template <class> class TPrintPolicy = TPrettyJsonPrintPolicy>
		FString ToString() const
		{
			FString Result;
			auto JsonWriter = TJsonWriterFactory<TCHAR, TPrintPolicy<TCHAR>>::Create(&Result);
			FJsonSerializer::Serialize(Array, JsonWriter);
			return Result;
		}

		template <template <class> class TPrintPolicy = TPrettyJsonPrintPolicy>
		FUtf8String ToUtf8String() const
		{
			FUtf8String Result;
			auto JsonWriter = TJsonWriterFactory<UTF8CHAR, TPrintPolicy<UTF8CHAR>>::Create(&Result);
			FJsonSerializer::Serialize(Array, JsonWriter);
			return Result;
		}

		int Num() const
		{
			return Array.Num();
		}

		FArray& Add(const FArray& Arr)            { Array.Emplace(Arr.AsJsonValue());                      return *this; }
		FArray& Add(const FObject& Obj)           { Array.Emplace(Obj.AsJsonValue());                      return *this; }

		template <class StringType
			UE_REQUIRES(std::is_constructible_v<FString, StringType>)>
		FArray& Add(const StringType& Str)        { Array.Emplace(MakeShared<FJsonValueString>(FString(Str))); return *this; }

		template <class NumberType
			UE_REQUIRES(!std::is_same_v<NumberType, bool> && (std::is_integral_v<NumberType> || std::is_floating_point_v<NumberType>))>
		FArray& Add(NumberType Number)            { Array.Emplace(MakeShared<FJsonValueNumber>(Number));   return *this; }

		template <class BoolType
			UE_REQUIRES(std::is_same_v<BoolType, bool>)>
		FArray& Add(BoolType Boolean)             { Array.Emplace(MakeShared<FJsonValueBoolean>(Boolean)); return *this; }
		FArray& Add(TYPE_OF_NULLPTR)              { Array.Emplace(MakeShared<FJsonValueNull>());           return *this; }

		FArray& Add(TSharedPtr<FJsonValue> Value) { Array.Emplace(Value);                                  return *this; }

		/** Add multiple values */
		template <class... ValueType
			UE_REQUIRES(sizeof...(ValueType) > 1)>
		FArray& Add(ValueType&&... Value)
		{
			// This should be implemented with a fold expression when our compilers support it
			int Temp[] = {0, (Add(Forward<ValueType>(Value)), 0)...};
			(void)Temp;
			return *this;
		}

		void CopyIf(const TArray<TSharedPtr<FJsonValue>>& Src, TFunctionRef<bool (const FJsonValue&)> Pred)
		{
			for (const TSharedPtr<FJsonValue>& Value: Src)
			{
				if (ensure(Value) && Pred(*Value))
				{
					Array.Emplace(Value);
				}
			}
		}
	private:
		TArray<TSharedPtr<FJsonValue>> Array;
	};
};
