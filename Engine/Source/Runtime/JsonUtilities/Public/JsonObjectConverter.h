// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Internationalization/Text.h"
#include "JsonGlobals.h"
#include "JsonObjectWrapper.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonTypes.h"
#include "Serialization/JsonWriter.h"
#include "Templates/SharedPointer.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "Templates/Models.h"
#include "Concepts/StaticClassProvider.h"

#define LOCTEXT_NAMESPACE "JsonObjectConverter"

enum class EJsonObjectConversionFlags
{
	None = 0,
	SkipStandardizeCase = 1 << 0,

	/**
	 * Write text in its complex exported format (eg, NSLOCTEXT(...)) rather than as a simple string.
	 * @note This is required to correctly support localization
	 */
	WriteTextAsComplexString = 1 << 1,

	/**
	 * suppress _ClassName key written for persistent objects that allows them to be
	 * deserialized later. This can be useful for times when writing to Json and you don't
	 * need to reserialize back into UObjects, as the _ClassName would be unnecessary.
	*/
	SuppressClassNameForPersistentObject = 1 << 2,
};

ENUM_CLASS_FLAGS(EJsonObjectConversionFlags)

class FProperty;
class UStruct;

/** Class that handles converting Json objects to and from UStructs */
class FJsonObjectConverter
{
public:

	/**
	 * FName case insensitivity can make the casing of UPROPERTIES unpredictable. Attempt to standardize output.
	 * See StandardizeCaseInPlace() for an in-place equivalent.
	 */
	[[nodiscard]] static JSONUTILITIES_API FString StandardizeCase(const FString &StringIn);

	static JSONUTILITIES_API void StandardizeCaseInPlace(FString& String);

	/** Parse an FText from a json object (assumed to be of the form where keys are culture codes and values are strings) */
	static JSONUTILITIES_API bool GetTextFromObject(const TSharedRef<FJsonObject>& Obj, FText& TextOut);

	/** Convert a Json value to text (takes some hints from the value name) */
	static JSONUTILITIES_API bool GetTextFromField(const FString& FieldName, const TSharedPtr<FJsonValue>& FieldValue, FText& TextOut);

public: // UStruct -> JSON

	/**
	 * Optional callback that will be run when exporting a single property to Json.
	 * If this returns a valid value it will be inserted into the export chain.
	 * If this returns nullptr or is not bound, it will try generic type-specific export behavior before falling back to outputting ExportText as a string.
	 */
	using CustomExportCallback = TDelegate<TSharedPtr<FJsonValue>(FProperty* Property, const void* Value)>;

	/**
	 * Optional callback that will be run when importing a single property from Json.
	 * If this returns true, it should have successfully turned the Json value into the property value.
	 * If this returns false or is not bound, it will try generic type-specific import behavior before failing.
	 */
	using CustomImportCallback = TDelegate<bool(const TSharedPtr<FJsonValue>& JsonValue, FProperty* Property, void* Value)>;

	static JSONUTILITIES_API const CustomExportCallback ExportCallback_WriteISO8601Dates;

	/**
	 * Templated version of UStructToJsonObject to try and make most of the params. Also serves as an example use case
	 *
	 * @param InStruct The UStruct instance to read from
	 * @param ExportCb Optional callback to override export behavior, if this returns null it will fallback to the default
	 * @param CheckFlags Only convert properties that match at least one of these flags. If 0 check all properties.
	 * @param SkipFlags Skip properties that match any of these flags
	 * @return FJsonObject pointer. Invalid if an error occurred.
	 */
	template<typename InStructType>
	static TSharedPtr<FJsonObject> UStructToJsonObject(const InStructType& InStruct, int64 CheckFlags = 0, int64 SkipFlags = 0, const CustomExportCallback* ExportCb = nullptr)
	{
		TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		if (UStructToJsonObject(InStructType::StaticStruct(), &InStruct, JsonObject, CheckFlags, SkipFlags, ExportCb))
		{
			return JsonObject;
		}
		return TSharedPtr<FJsonObject>(); // something went wrong
	}

	/**
	 * Converts from a UStruct to a Json Object, using exportText
	 *
	 * @param StructDefinition UStruct definition that is looked over for properties
	 * @param Struct The UStruct instance to copy out of
	 * @param OutJsonObject Json Object to be filled in with data from the ustruct
	 * @param CheckFlags Only convert properties that match at least one of these flags. If 0 check all properties.
	 * @param SkipFlags Skip properties that match any of these flags
	 * @param ExportCb Optional callback to override export behavior, if this returns null it will fallback to the default
	 * @param ConversionFlags Bitwise flags to customize the conversion behavior
	 *
	 * @return False if any properties failed to write
	 */
	static JSONUTILITIES_API bool UStructToJsonObject(const UStruct* StructDefinition, const void* Struct, TSharedRef<FJsonObject> OutJsonObject, int64 CheckFlags = 0, int64 SkipFlags = 0, const CustomExportCallback* ExportCb = nullptr, EJsonObjectConversionFlags ConversionFlags = EJsonObjectConversionFlags::None);

	/**
	 * Converts from a UStruct to a json string containing an object, using exportText
	 *
	 * @param StructDefinition UStruct definition that is looked over for properties
	 * @param Struct The UStruct instance to copy out of
	 * @param JsonObject Json Object to be filled in with data from the ustruct
	 * @param CheckFlags Only convert properties that match at least one of these flags. If 0 check all properties.
	 * @param SkipFlags Skip properties that match any of these flags
	 * @param Indent How many tabs to add to the json serializer
	 * @param ExportCb Optional callback to override export behavior, if this returns null it will fallback to the default
	 * @param bPrettyPrint Option to use pretty print (e.g., adds line endings) or condensed print
	 *
	 * @return False if any properties failed to write
	 */
	static JSONUTILITIES_API bool UStructToJsonObjectString(const UStruct* StructDefinition, const void* Struct, FString& OutJsonString, int64 CheckFlags = 0, int64 SkipFlags = 0, int32 Indent = 0, const CustomExportCallback* ExportCb = nullptr, bool bPrettyPrint = true);

	/**
	 * Templated version; Converts from a UStruct to a json string containing an object, using exportText
	 *
	 * @param InStruct The UStruct instance to copy out of
	 * @param OutJsonString Json Object to be filled in with data from the ustruct
	 * @param CheckFlags Only convert properties that match at least one of these flags. If 0 check all properties.
	 * @param SkipFlags Skip properties that match any of these flags
	 * @param Indent How many tabs to add to the json serializer
	 * @param ExportCb Optional callback to override export behavior, if this returns null it will fallback to the default
	 * @param bPrettyPrint Option to use pretty print (e.g., adds line endings) or condensed print
	 *
	 * @return False if any properties failed to write
	 */
	template<typename InStructType>
	static bool UStructToJsonObjectString(const InStructType& InStruct, FString& OutJsonString, int64 CheckFlags = 0, int64 SkipFlags = 0, int32 Indent = 0, const CustomExportCallback* ExportCb = nullptr, bool bPrettyPrint = true)
	{
		if constexpr (TModels<CStaticClassProvider, InStructType>::Value)
		{
			return UStructToJsonObjectString(InStructType::StaticClass(), &InStruct, OutJsonString, CheckFlags, SkipFlags, Indent, ExportCb, bPrettyPrint);
		}
		else
		{
			return UStructToJsonObjectString(InStructType::StaticStruct(), &InStruct, OutJsonString, CheckFlags, SkipFlags, Indent, ExportCb, bPrettyPrint);
		}
	}

	/**
	 * Wrapper to UStructToJsonObjectString that allows a print policy to be specified.
	 */
	template<typename CharType, template<typename> class PrintPolicy>
	static bool UStructToFormattedJsonObjectString(const UStruct* StructDefinition, const void* Struct, FString& OutJsonString, int64 CheckFlags = 0, int64 SkipFlags = 0, int32 Indent = 0, const CustomExportCallback* ExportCb = nullptr, EJsonObjectConversionFlags ConversionFlags = EJsonObjectConversionFlags::None)
	{
		TSharedRef<FJsonObject> JsonObject = MakeShareable(new FJsonObject());
		if (UStructToJsonObject(StructDefinition, Struct, JsonObject, CheckFlags, SkipFlags, ExportCb, ConversionFlags))
		{
			TSharedRef<TJsonWriter<CharType, PrintPolicy<CharType>>> JsonWriter = TJsonWriterFactory<CharType, PrintPolicy<CharType>>::Create(&OutJsonString, Indent);

			if (FJsonSerializer::Serialize(JsonObject, JsonWriter))
			{
				JsonWriter->Close();
				return true;
			}
			else
			{
				UE_LOGF(LogJson, Warning, "UStructToFormattedObjectString - Unable to write out json");
				JsonWriter->Close();
			}
		}

		return false;
	}

	/**
	 * Converts from a UStruct to a set of json attributes (possibly from within a JsonObject)
	 *
	 * @param StructDefinition UStruct definition that is looked over for properties
	 * @param Struct The UStruct instance to copy out of
	 * @param OutJsonAttributes Map of attributes to copy in to
	 * @param CheckFlags Only convert properties that match at least one of these flags. If 0 check all properties.
	 * @param SkipFlags Skip properties that match any of these flags
	 * @param ExportCb Optional callback to override export behavior, if this returns null it will fallback to the default
	 * @param ConversionFlags Bitwise flags to customize the conversion behavior
	 *
	 * @return False if any properties failed to write
	 */
	static JSONUTILITIES_API bool UStructToJsonAttributes(const UStruct* StructDefinition, const void* Struct, TSharedRef<FJsonObject> OutJsonObject, int64 CheckFlags = 0, int64 SkipFlags = 0, const CustomExportCallback* ExportCb = nullptr, EJsonObjectConversionFlags ConversionFlags = EJsonObjectConversionFlags::None);

	/* * Converts from a FProperty to a Json Value using exportText
	 *
	 * @param Property			The property to export
	 * @param Value				Pointer to the value of the property
	 * @param CheckFlags		Only convert properties that match at least one of these flags. If 0 check all properties.
	 * @param SkipFlags			Skip properties that match any of these flags
	 * @param ExportCb Optional callback to override export behavior, if this returns null it will fallback to the default
	 * @param OuterProperty		If applicable, the Array/Set/Map Property that contains this property
	 * @param ConversionFlags	Bitwise flags to customize the conversion behavior
	 *
	 * @return					The constructed JsonValue from the property
	 */
	static JSONUTILITIES_API TSharedPtr<FJsonValue> UPropertyToJsonValue(FProperty* Property, const void* Value, int64 CheckFlags = 0, int64 SkipFlags = 0, const CustomExportCallback* ExportCb = nullptr, FProperty* OuterProperty = nullptr, EJsonObjectConversionFlags ConversionFlags = EJsonObjectConversionFlags::None);

public: // JSON -> UStruct

	/**
	 * Converts from a Json Object to a UStruct, using importText
	 *
	 * @param JsonObject Json Object to copy data out of
	 * @param StructDefinition UStruct definition that is looked over for properties
	 * @param OutStruct The UStruct instance to copy in to
	 * @param CheckFlags Only convert properties that match at least one of these flags. If 0 check all properties.
	 * @param SkipFlags Skip properties that match any of these flags
	 * @param bStrictMode Whether to strictly check the json attributes
	 * @param OutFailReason Reason of the failure if any
	 * @param ImportCb Optional callback to override import behaviour, if this returns false it will fallback to the default
	 *
	 * @return False if any properties matched but failed to deserialize
	 */
	static JSONUTILITIES_API bool JsonObjectToUStruct(const TSharedRef<FJsonObject>& JsonObject, const UStruct* StructDefinition, void* OutStruct, int64 CheckFlags = 0, int64 SkipFlags = 0, const bool bStrictMode = false, FText* OutFailReason = nullptr, const CustomImportCallback* ImportCb = nullptr);

	/**
	 * Templated version of JsonObjectToUStruct
	 *
	 * @param JsonObject Json Object to copy data out of
	 * @param OutStruct The UStruct instance to copy in to
	 * @param CheckFlags Only convert properties that match at least one of these flags. If 0 check all properties.
	 * @param SkipFlags Skip properties that match any of these flags
	 * @param bStrictMode Whether to strictly check the json attributes
	 * @param OutFailReason Reason of the failure if any
	 * @param ImportCb Optional callback to override import behaviour, if this returns false it will fallback to the default
	 *
	 * @return False if any properties matched but failed to deserialize
	 */
	template<typename OutStructType>
	static bool JsonObjectToUStruct(const TSharedRef<FJsonObject>& JsonObject, OutStructType* OutStruct, int64 CheckFlags = 0, int64 SkipFlags = 0, const bool bStrictMode = false, FText* OutFailReason = nullptr, const CustomImportCallback* ImportCb = nullptr)
	{
		if constexpr (TModels<CStaticClassProvider, OutStructType>::Value)
		{
			return JsonObjectToUStruct(JsonObject, OutStructType::StaticClass(), OutStruct, CheckFlags, SkipFlags, bStrictMode, OutFailReason, ImportCb);
		}
		else
		{
			return JsonObjectToUStruct(JsonObject, OutStructType::StaticStruct(), OutStruct, CheckFlags, SkipFlags, bStrictMode, OutFailReason, ImportCb);
		}
	}

	/**
	 * Converts a set of json attributes (possibly from within a JsonObject) to a UStruct, using importText
	 *
	 * @param JsonAttributes Json Object to copy data out of
	 * @param StructDefinition UStruct definition that is looked over for properties
	 * @param OutStruct The UStruct instance to copy in to
	 * @param CheckFlags Only convert properties that match at least one of these flags. If 0 check all properties.
	 * @param SkipFlags Skip properties that match any of these flags
	 * @param bStrictMode Whether to strictly check the json attributes
	 * @param OutFailReason Reason of the failure if any
	 * @param ImportCb Optional callback to override import behaviour, if this returns false it will fallback to the default
	 *
	 * @return False if any properties matched but failed to deserialize
	 */
	static JSONUTILITIES_API bool JsonAttributesToUStruct(const TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& JsonAttributes, const UStruct* StructDefinition, void* OutStruct, int64 CheckFlags = 0, int64 SkipFlags = 0, const bool bStrictMode = false, FText* OutFailReason = nullptr, const CustomImportCallback* ImportCb = nullptr);

	/**
	 * Converts a single JsonValue to the corresponding FProperty (this may recurse if the property is a UStruct for instance).
	 *
	 * @param JsonValue The value to assign to this property
	 * @param Property The FProperty definition of the property we're setting.
	 * @param OutValue Pointer to the property instance to be modified.
	 * @param CheckFlags Only convert sub-properties that match at least one of these flags. If 0 check all properties.
	 * @param SkipFlags Skip sub-properties that match any of these flags
	 * @param bStrictMode Whether to strictly check the json attributes
	 * @param OutFailReason Reason of the failure if any
	 * @param ImportCb Optional callback to override import behaviour, if this returns false it will fallback to the default
	 *
	 * @return False if the property failed to serialize
	 */
	static JSONUTILITIES_API bool JsonValueToUProperty(const TSharedPtr<FJsonValue>& JsonValue, FProperty* Property, void* OutValue, int64 CheckFlags = 0, int64 SkipFlags = 0, const bool bStrictMode = false, FText* OutFailReason = nullptr, const CustomImportCallback* ImportCb = nullptr);

	/**
	 * Converts from a json string containing an object to a UStruct
	 *
	 * @param JsonString String containing JSON formatted data.
	 * @param OutStruct The UStruct instance to copy in to
	 * @param CheckFlags Only convert properties that match at least one of these flags. If 0 check all properties.
	 * @param SkipFlags Skip properties that match any of these flags
	 * @param bStrictMode Whether to strictly check the json attributes
	 * @param OutFailReason Reason of the failure if any
	 * @param ImportCb Optional callback to override import behaviour, if this returns false it will fallback to the default
	 *
	 * @return False if any properties matched but failed to deserialize
	 */
	template<typename OutStructType>
	static bool JsonObjectStringToUStruct(const FString& JsonString, OutStructType* OutStruct, int64 CheckFlags = 0, int64 SkipFlags = 0, const bool bStrictMode = false, FText* OutFailReason = nullptr, const CustomImportCallback* ImportCb = nullptr)
	{
		TSharedPtr<FJsonObject> JsonObject;
		TSharedRef<TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(JsonString);
		if (!FJsonSerializer::Deserialize(JsonReader, JsonObject) || !JsonObject.IsValid())
		{
			UE_LOGF(LogJson, Warning, "JsonObjectStringToUStruct - Unable to parse. json=[%ls]", *JsonString);
			if (OutFailReason)
			{
				*OutFailReason = FText::Format(LOCTEXT("FailJsonObjectDeserialize", "JsonObjectStringToUStruct - Unable to parse. json=[{0}]"), FText::FromString(*JsonString));
			}
			return false;
		}
		if (!FJsonObjectConverter::JsonObjectToUStruct(JsonObject.ToSharedRef(), OutStruct, CheckFlags, SkipFlags, bStrictMode, OutFailReason, ImportCb))
		{
			UE_LOGF(LogJson, Warning, "JsonObjectStringToUStruct - Unable to deserialize. json=[%ls]", *JsonString);
			if (OutFailReason)
			{
				*OutFailReason = FText::Format(LOCTEXT("FailJsonObjectConversion", "JsonObjectStringToUStruct - Unable to deserialize. json=[{0}]\n{1}"), FText::FromString(*JsonString), *OutFailReason);
			}
			return false;
		}
		return true;
	}

	/**
	 * Converts from a json string containing an array to an array of UStructs
	 *
	 * @param JsonString String containing JSON formatted data.
	 * @param OutStructArray The UStruct array to copy in to
	 * @param CheckFlags Only convert properties that match at least one of these flags. If 0 check all properties.
	 * @param SkipFlags Skip properties that match any of these flags.
	 * @param bStrictMode Whether to strictly check the json attributes
	 * @param OutFailReason Reason of the failure if any
	 * @param ImportCb Optional callback to override import behaviour, if this returns false it will fallback to the default
	 *
	 * @return False if any properties matched but failed to deserialize.
	 */
	template<typename OutStructType>
	static bool JsonArrayStringToUStruct(const FString& JsonString, TArray<OutStructType>* OutStructArray, int64 CheckFlags = 0, int64 SkipFlags = 0, const bool bStrictMode = false, FText* OutFailReason = nullptr, const CustomImportCallback* ImportCb = nullptr)
	{
		TArray<TSharedPtr<FJsonValue> > JsonArray;
		TSharedRef<TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(JsonString);
		if (!FJsonSerializer::Deserialize(JsonReader, JsonArray))
		{
			UE_LOGF(LogJson, Warning, "JsonArrayStringToUStruct - Unable to parse. json=[%ls]", *JsonString);
			if (OutFailReason)
			{
				*OutFailReason = FText::Format(LOCTEXT("FailJsonArrayDeserialize", "JsonArrayStringToUStruct - Unable to parse. json=[{0}]"), FText::FromString(*JsonString));
			}
			return false;
		}
		if (!JsonArrayToUStruct(JsonArray, OutStructArray, CheckFlags, SkipFlags, bStrictMode, OutFailReason, ImportCb))
		{
			UE_LOGF(LogJson, Warning, "JsonArrayStringToUStruct - Error parsing one of the elements. json=[%ls]", *JsonString);
			if (OutFailReason)
			{
				*OutFailReason = FText::Format(LOCTEXT("FailJsonArrayConversion", "JsonArrayStringToUStruct - Error parsing one of the elements. json=[{0}]\n{1}"), FText::FromString(*JsonString), *OutFailReason);
			}
			return false;
		}
		return true;
	}

	/**
	 * Converts from an array of json values to an array of UStructs.
	 *
	 * @param JsonArray Array containing json values to convert.
	 * @param OutStructArray The UStruct array to copy in to
	 * @param CheckFlags Only convert properties that match at least one of these flags. If 0 check all properties.
	 * @param SkipFlags Skip properties that match any of these flags.
	 * @param bStrictMode Whether to strictly check the json attributes
	 * @param OutFailReason Reason of the failure if any
	 * @param ImportCb Optional callback to override import behaviour, if this returns false it will fallback to the default
	 *
	 * @return False if any of the matching elements are not an object, or if one of the matching elements could not be converted to the specified UStruct type.
	 */
	template<typename OutStructType>
	static bool JsonArrayToUStruct(const TArray<TSharedPtr<FJsonValue>>& JsonArray, TArray<OutStructType>* OutStructArray, int64 CheckFlags = 0, int64 SkipFlags = 0, const bool bStrictMode = false, FText* OutFailReason = nullptr, const CustomImportCallback* ImportCb = nullptr)
	{
		OutStructArray->SetNum(JsonArray.Num());
		for (int32 i = 0; i < JsonArray.Num(); ++i)
		{
			const auto& Value = JsonArray[i];
			if (Value->Type != EJson::Object)
			{
				UE_LOGF(LogJson, Warning, "JsonArrayToUStruct - Array element [%i] was not an object.", i);
				if (OutFailReason)
				{
					*OutFailReason = FText::Format(LOCTEXT("FailJsonArrayElementObject", "JsonArrayToUStruct - Array element [{0}] was not an object."), i);
				}
				return false;
			}
			if (!FJsonObjectConverter::JsonObjectToUStruct(Value->AsObject().ToSharedRef(), OutStructType::StaticStruct(), &(*OutStructArray)[i], CheckFlags, SkipFlags, bStrictMode, OutFailReason, ImportCb))
			{
				UE_LOGF(LogJson, Warning, "JsonArrayToUStruct - Unable to convert element [%i].", i);
				if (OutFailReason)
				{
					*OutFailReason = FText::Format(LOCTEXT("FailJsonArrayElementConversion", "JsonArrayToUStruct - Unable to convert element [{0}].\n{1}"), i, *OutFailReason);
				}
				return false;
			}
		}
		return true;
	}

	/*
	* Parses text arguments from Json into a map
	* @param JsonObject Object to parse arguments from
	*/
	static JSONUTILITIES_API FFormatNamedArguments ParseTextArgumentsFromJson(const TSharedPtr<const FJsonObject>& JsonObject);
};

#undef LOCTEXT_NAMESPACE
