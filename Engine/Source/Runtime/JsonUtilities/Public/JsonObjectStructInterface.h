// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"

class FJsonObject;
struct IJsonObjectStructConverter;

/** Global registry for type-registered static-interface converters to override default struct behavior in FJsonObjectConverter. */
struct FJsonObjectStructInterfaceRegistry
{
	static JSONUTILITIES_API void RegisterStructConverter(const UScriptStruct* ScriptStruct, const IJsonObjectStructConverter* ConverterInterface);
	static JSONUTILITIES_API void UnregisterStructConverter(const UScriptStruct* ScriptStruct);

	/* Return whether or not a struct converter has been registered for this ScriptStruct */
	static JSONUTILITIES_API bool HasStructConverterRegistered(const UScriptStruct* ScriptStruct);
};

enum class EJsonObjectConvertResult
{
	/** No conversion was performed. Fallback to try the default JsonObjectConverter instead. */
	UseDefaultConverter,
	/** No conversion is possible. Abandon further efforts. Ask caller to fail. */
	FailAndAbort,
	/** No conversion was performed. Don't try any fallback. Allow caller to still succeed the conversion (if this was a nested convert attempt). */
	IgnoreAndContinue,
	/** Conversion of the struct was performed. */
	Converted,
};

/**
 * UStructs are special and may need to avoid the use of virtual functions via standard interface patterns due to mismatched base-class polymorphism.
 * So use a static TImplementsJsonObjectStructConverter<StructType> which uses a static-interface pattern with global type-registry instead of implementing an interface directly.
 *
 * The registered struct should have the following functions (the compiler will fail without them):
 * EJsonObjectConvertResult ConvertToJson(TSharedPtr<FJsonObject>& OutJsonObject) const;
 * EJsonObjectConvertResult ConvertFromJson(const TSharedPtr<FJsonObject>& InJsonObject);
 */
struct IJsonObjectStructConverter
{
	/** 
	* Result of Converted represents success by the interface call,
	* FailAndAbort will bubble up the failure to its owner converter which may fail the entire converter tree,
	* IgnoreAndContinue will make an empty FJsonObject for this struct and the converter tree can still succeed,
	* and UseDefaultConverter will try to fallback to the non-static-interface behavior instead.
	*/
	virtual EJsonObjectConvertResult ConvertToJson(const void* StructMemory, TSharedPtr<FJsonObject>& OutJsonObject) const = 0;

	/** 
	* Result of Converted represents success by the interface call,
	* FailAndAbort will bubble up the failure to its owner converter failing the entire converter tree, 
	* IgnoreAndContinue should leave the struct untouched/default and the converter tree can still succeed, 
	* and UseDefaultConverter will try to fallback to the non-static-interface behavior instead. 
	*/
	virtual EJsonObjectConvertResult ConvertFromJson(void* StructMemory, const TSharedPtr<FJsonObject>& InJsonObject) const = 0;

protected:

	virtual ~IJsonObjectStructConverter() = default;
};

/** 
* This template is to be statically constructed and passed as a parameter to RegisterStructConverter in StartupModule() for the specified StructType. 
* Remember to explicitly call UnregisterStructConverter in ShutdownModule() as well.
* 
* Example usage:
* 
* void FExampleModule::StartupModule()
* {
* 	static const TImplementsJsonObjectStructConverter<FExampleStruct> ExampleConverter = TImplementsJsonObjectStructConverter<FExampleStruct>();
* 	FJsonObjectStructInterfaceRegistry::RegisterStructConverter(FExampleStruct::StaticStruct(), &ExampleConverter);
* }
* 
* void FExampleModule::ShutdownModule()
* {
* 	FJsonObjectStructInterfaceRegistry::UnregisterStructConverter(FExampleStruct::StaticStruct());
* }
*/
template<typename StructType>
struct TImplementsJsonObjectStructConverter : public IJsonObjectStructConverter
{
	TImplementsJsonObjectStructConverter<StructType>() {};

	virtual EJsonObjectConvertResult ConvertToJson(const void* StructMemory, TSharedPtr<FJsonObject>& OutJsonObject) const override
	{
		return static_cast<const StructType*>(StructMemory)->ConvertToJson(OutJsonObject);
	}

	virtual EJsonObjectConvertResult ConvertFromJson(void* StructMemory, const TSharedPtr<FJsonObject>& InJsonObject) const override
	{
		return static_cast<StructType*>(StructMemory)->ConvertFromJson(InJsonObject);
	}
};

/** Functions for internal use. */
namespace UE::Json::Private
{
	const struct IJsonObjectStructConverter* GetStructConverterInterface(const UScriptStruct* ScriptStruct);
}