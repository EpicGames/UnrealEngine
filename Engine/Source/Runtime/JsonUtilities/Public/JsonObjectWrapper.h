// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "JsonObjectWrapper.generated.h"

class FArchive;
class FJsonObject;
class FOutputDevice;

/** UStruct that holds a JsonObject, can be used by structs passed to JsonObjectConverter to pass through JsonObjects directly */
USTRUCT(BlueprintType, meta = (DisplayName = "JsonObject"))
struct FJsonObjectWrapper
{
	GENERATED_USTRUCT_BODY()
	
public:
	JSONUTILITIES_API FJsonObjectWrapper();
	
	UPROPERTY(EditAnywhere, Category = "JSON")
	FString JsonString;

	TSharedPtr<FJsonObject> JsonObject;

	JSONUTILITIES_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);
	JSONUTILITIES_API bool ExportTextItem(FString& ValueStr, FJsonObjectWrapper const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	JSONUTILITIES_API void PostSerialize(const FArchive& Ar);

	JSONUTILITIES_API explicit operator bool() const;

	JSONUTILITIES_API bool JsonObjectToString(FString& Str) const;
	JSONUTILITIES_API bool JsonObjectFromString(const FString& Str);

	/**
	 * Returns the memory allocated by this object in Bytes, should NOT include sizeof(*this).
	 */
	JSONUTILITIES_API SIZE_T GetAllocatedSize() const;
};

template<>
struct TStructOpsTypeTraits<FJsonObjectWrapper> : public TStructOpsTypeTraitsBase2<FJsonObjectWrapper>
{
	enum
	{
		WithImportTextItem = true,
		WithExportTextItem = true,
		WithPostSerialize = true,
	};
};
UCLASS()
class UJsonUtilitiesDummyObject : public UObject
{
	GENERATED_BODY()
};
