// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "CoreMinimal.h"
#include "Dom/JsonObject.h"

#include "JsonSchema/JsonSchemaMemberPath.h"

/**
 * Used for constructing a member path for properties, and storing/accessing metadata for multiple
 * property member paths. Also stores metadata for a root struct.
 */
struct FJsonSchemaEditorMetadata
{
	using FJsonSchemaPropertyMemberPathToPropertyMetadataMap = TMap<FString, TSharedPtr<FJsonObject>>;
	
	JSONUTILITIES_API FJsonSchemaEditorMetadata();
	
	JSONUTILITIES_API FJsonSchemaEditorMetadata(const FJsonSchemaEditorMetadata& Other);
	
	JSONUTILITIES_API FJsonSchemaEditorMetadata& operator=(const FJsonSchemaEditorMetadata& Other);

	/**
	 * Top-level metadata (if any) for the root struct only. Mainly used when UStruct is UFunction.
	 */
	TSharedPtr<FJsonObject> RootStructMetadata;
		
	/**
	 * Adds property metadata for the current property member path.
	 * @param PropertyMetadata Property metadata.
	 */
	JSONUTILITIES_API void SetPropertyMetadataForCurrentPropertyMemberPath(const TSharedRef<FJsonObject>& PropertyMetadata);

	/**
	 * Gets property metadata for the current property member path.
	 * @return Property metadata for the current property member path.
	 */
	JSONUTILITIES_API TSharedPtr<FJsonObject> GetPropertyMetadataForCurrentPropertyMemberPath();

	/**
	 * Get Property member path -to- property metadata map.
	 */
	JSONUTILITIES_API const FJsonSchemaPropertyMemberPathToPropertyMetadataMap& GetPropertyMemberPathToPropertyMetadataMap() const;
	JSONUTILITIES_API FJsonSchemaPropertyMemberPathToPropertyMetadataMap& GetPropertyMemberPathToPropertyMetadataMap();
	
	/**
	 * Current property member path being constructed.
	 */
	FJsonSchemaMemberPath CurrentPropertyMemberPath;

private:

	/**
	 * Maps completed property member paths to property metadata, if present.
	 * Per-property metadata, including inner properties e.g: Person, Person.Name, Person.Age
	 */
	FJsonSchemaPropertyMemberPathToPropertyMetadataMap PropertyMemberPathToPropertyMetadataMap;
};
