// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "CoreMinimal.h"
#include "Dom/JsonObject.h"
#include "Internationalization/Text.h"
#include "UObject/Class.h"
#include "Templates/SharedPointer.h"

#include "JsonSchemaPropertyFilter.h"
#include "JsonSchemaEditorMetadata.h"

/**
 * Used for generating a JSON schema from any UStruct or FProperty. 
 */
class FJsonSchemaGenerator
{
public:
	
	/**
	 * Derives a JSON Schema definition for Property's type.
	 *
	 * Can be called by a developer directly, but primarily used by UStructToJsonSchemaObject().
	 * Attention -
	 *		- Property name is not part of the returned schema.
	 *		- If calling in Editor context, metadata will be included. Otherwise, it will not. (i.e. default and min/max clamp values.)
	 *		- If Property's owner Struct is a UFunction and calling from an Editor context, metadata will only include default values if UFunction is
	 *		  a Blueprint function.
	 * 
	 * If Property is a struct or container of structs property etc, the nested types will be
	 * recursed into using UStructToJsonSchemaObject to derive a fully complete schema.  
	 *
	 * Importantly, the derived JSON Schema describes the JSON which would be produced by FJsonObjectConverter::UPropertyToJsonValue for a property 
	 * value of this type. And conversely, any JSON adhering to this JSON Schema, may then be deserialized to this property,
	 * using FJsonObjectConverter::JsonValueToUProperty.
	 *
	 * e.g:
	 *
	 *		FBoolProperty -> { "type": "boolean" }
	 *		FStrProperty & FTextProperty -> { "type": "string" }
	 *		FStructProperty -> { "type": "object", "properties": { ... } }
	 * 
	 * @param Property The property to build a schema for.
	 * @param PropertyFilter Determines how to filter collected struct and property data.
	 * @param CachedEditorMetadata If valid, use this metadata instead of collecting it. Otherwise, it will be automatically collected
	 *		(if running in Editor context.) This is optional, and used for workflows that have cached metadata from an earlier process.
	 * @param InstanceMemory When set allows specs to be generated from FInstancedStruct and FInstancedPropertyBag.
	 * @return Schema for the FProperty, as a JSON object.
	 */
	JSONUTILITIES_API TSharedPtr<FJsonObject> static FPropertyToJsonSchemaObject(TNotNull<const FProperty*> Property,
		const FJsonSchemaPropertyFilter& PropertyFilter = FJsonSchemaPropertyFilter(),
		const FJsonSchemaEditorMetadata* CachedEditorMetadata = nullptr,
		const void* InstanceMemory = nullptr);
	
	/**
	 * String output variant of FPropertyToJsonSchemaObject() (see above.)
	 * 
	 * Provides JSON string instead of JSON object.
	 * 
	 * @param Property The property to build a schema for.
	 * @param PropertyFilter Determines how to filter collected struct and property data.
	 * @param CachedEditorMetadata If valid, use this metadata instead of collecting it. Otherwise, it will be automatically collected
	 *		(if running in Editor context.) This is optional, and used for workflows that have cached metadata from an earlier process.
	 * @param Indent How many additional tabs to add to the json serializer output.
	 * @param bPrettyPrint Option to use pretty print (e.g., adds line endings), otherwise condensed print.
	 * @return Schema for the property, as a JSON string.
	 */
	JSONUTILITIES_API static FString FPropertyToJsonSchemaString(TNotNull<const FProperty*> Property,
		const FJsonSchemaPropertyFilter& PropertyFilter = FJsonSchemaPropertyFilter(), 
		const FJsonSchemaEditorMetadata* CachedEditorMetadata = nullptr,
		const int32 Indent = 0, const bool bPrettyPrint = true);
	
	/**
	 * Recursively walks Struct's property hierarchy to derive a JSON Schema definition for this type.
	 *
	 * To be called by a developer directly.
	 * Attention -
	 *		- Struct name is not part of the returned schema.
	 *		- If calling in Editor context, metadata will be included. Otherwise, it will not. (i.e. struct's description)
	 *		- If Struct is a UFunction, and calling from an Editor context, metadata will only include default values for the function parameters if 
	 *		  the function is a Blueprint function.
	 *		- If Struct is a UClass, its UFunctions will not be included in the returned schema. TODO - Need a UClass version of this function? 
	 * 
	 * Importantly, the derived JSON Schema describes the JSON which would be produced by FJsonObjectConverter::StructToJsonObject for an instance
	 * of the given UStruct. And conversely, any JSON adhering to this JSON Schema, may then be deserialized to an instance of Struct,
	 * using FJsonObjectConverter::JsonObjectToUStruct.
	 *
	 * e.g:
	 *
	 *		// Details for a person
	 *		USTRUCT()
	 *		struct FPerson
	 *		{
	 *			GENERATED_BODY()
	 *	
	 *			// Full name
	 *			UPROPERTY()
	 *			FString Name;
	 *
	 *			// How old they are
	 *			UPROPERTY(meta = (ClampMin = "1"))
	 *			int32 Age;
	 *		}
	 *
	 *		// A family of people
	 *		USTRUCT()
	 *		struct FFamily
	 *		{
	 *			// Members of this family
	 *			UPROPERTY()
	 *			TArray<FPerson> Members; 
	 *		}
	 *
	 *		->
	 *		
	 *		{
	 *			"type": "object",
	 *			"properties": {
	 *				"members": {
	 *					"type": "array",
	 *					"description": "Members of this family",
	 *					"items": {
	 *						"type": "object",
	 *						"properties": {
	 *							"name": {
	 *								"type": "string",
	 *								"description": "Full name"
	 *							},
	 *							"age": {
	 *								"type": "integer",
	 *								"description": "How old they are",
	 *								"minimum": 1
	 *							}
	 *						},
	 *						"required": ["name", "age"]
	 *					}
	 *				}
	 *			},
	 *			"required": ["members"]
	 *		}
	 *
	 * IMPORTANT - If called while in Editor, schema will include metadata. Otherwise, it will not.
	 * 
	 * @param Struct The type to build a schema for. Can be either UScriptStruct, UClass or UFunction.
	 * @param PropertyFilter Determines how to filter collected struct and property data.
	 * @param CachedEditorMetadata If valid, use this metadata instead of collecting it. Otherwise, it will be automatically collected
	 *		(if running in Editor context.) This is optional, and used for workflows that have cached metadata from an earlier process.
	 * @param InstanceMemory When set allows specs to be generated from FInstancedStruct and FInstancedPropertyBag.
	 * @return Schema for the UStruct, as a JSON object.
	 */
	JSONUTILITIES_API static TSharedPtr<FJsonObject> UStructToJsonSchemaObject(TNotNull<const UStruct*> Struct,
		const FJsonSchemaPropertyFilter& PropertyFilter = FJsonSchemaPropertyFilter(),
		const FJsonSchemaEditorMetadata* CachedEditorMetadata = nullptr,
		const void* InstanceMemory = nullptr);
	
	/**
	 * Templated variant of UStructToJsonSchemaObject() (see above.).
	 * 
	 * Works according to struct/class type instead of an taking actual UStruct/UClass reference -
	 * i.e. UStructToJsonSchema<FSomeStruct>() or UStructToJsonSchema<USomeClass>().
	 * 
	 * @param PropertyFilter Determines how to filter collected struct and property data.
	 * @param CachedEditorMetadata If valid, use this metadata instead of collecting it. Otherwise, it will be automatically collected
	 *		(if running in Editor context.) This is optional, and used for workflows that have cached metadata from an earlier process.
	 * @return Schema for the UStruct, as a JSON object.
	 */
	template<typename InStructType>
	static TSharedPtr<FJsonObject> UStructToJsonSchemaObject(
		const FJsonSchemaPropertyFilter& PropertyFilter = FJsonSchemaPropertyFilter(),
		FJsonSchemaEditorMetadata* CachedEditorMetadata = nullptr)
	{
		// Compile-time discovery if struct type has StaticClass(), so is UClass.
		if constexpr (TModels<CStaticClassProvider, InStructType>::Value)
		{
			return UStructToJsonSchemaObject(InStructType::StaticClass(), PropertyFilter, CachedEditorMetadata);
		}
		else
		{
			return UStructToJsonSchemaObject(InStructType::StaticStruct(), PropertyFilter, CachedEditorMetadata);
		}
	}

	/**
	 * String output variant of UStructToJsonSchemaObject() (see above.)
	 * 
	 * Provides JSON string instead of JSON object.
	 *
	 * @param Struct The type to build a schema for. Can be either UScriptStruct, UClass or UFunction.
	 * @param PropertyFilter Determines how to filter collected struct and property data.
	 * @param CachedEditorMetadata If valid, use this metadata instead of collecting it. Otherwise, it will be automatically collected
	 *		(if running in Editor context.) This is optional, and used for workflows that have cached metadata from an earlier process.
	 * @param Indent How many additional tabs to add to the json serializer output.
	 * @param bPrettyPrint Option to use pretty print (e.g., adds line endings), otherwise condensed print.
	 * @return Schema for the UStruct, as a JSON string.
	 */
	JSONUTILITIES_API static FString UStructToJsonSchemaString(TNotNull<const UStruct*> Struct,
		const FJsonSchemaPropertyFilter& PropertyFilter = FJsonSchemaPropertyFilter(),
		const FJsonSchemaEditorMetadata* CachedEditorMetadata = nullptr,
		const int32 Indent = 0, const bool bPrettyPrint = true);
	
	/**
	 * Templated variant of UStructToJsonSchemaString() (see above.).
	 * 
	 * Works according to struct/class type instead of an taking actual UStruct/UClass reference -
	 * i.e. UStructToJsonSchema<FSomeStruct>() or UStructToJsonSchema<USomeClass>().
	 * 
	 * @param PropertyFilter Determines how to filter collected struct and property data.
	 * @param CachedEditorMetadata If valid, use this metadata instead of collecting it. Otherwise, it will be automatically collected
	 *		(if running in Editor context.) This is optional, and used for workflows that have cached metadata from an earlier process.
	 * @param Indent How many additional tabs to add to the json serializer output.
	 * @param bPrettyPrint Option to use pretty print (e.g., adds line endings), otherwise condensed print.
	 * @return Schema for the UStruct, as a JSON string.
	 */
	template<typename InStructType>
	static FString UStructToJsonSchemaString(
		const FJsonSchemaPropertyFilter& PropertyFilter = FJsonSchemaPropertyFilter(),
		const FJsonSchemaEditorMetadata* CachedEditorMetadata = nullptr,
		const int32 Indent = 0, const bool bPrettyPrint = true)
	{
		// Compile-time discovery if struct type has StaticClass(), so is UClass.
		if constexpr (TModels<CStaticClassProvider, InStructType>::Value)
		{
			return UStructToJsonSchemaString(*InStructType::StaticClass(), PropertyFilter, CachedEditorMetadata, Indent, bPrettyPrint);
		}
		else
		{
			return UStructToJsonSchemaString(*InStructType::StaticStruct(), PropertyFilter, CachedEditorMetadata, Indent, bPrettyPrint);
		}
	}
};
