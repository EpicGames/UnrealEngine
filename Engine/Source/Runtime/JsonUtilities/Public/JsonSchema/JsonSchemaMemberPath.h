// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "CoreMinimal.h"

/**
 * JSON member path builder.
 */
class FJsonSchemaMemberPath
{
public:

	/**
	 * Text used to represent divider between member names and sub-member names.
	 */
	JSONUTILITIES_API static const FStringView Delimiter;

	JSONUTILITIES_API FJsonSchemaMemberPath();
	JSONUTILITIES_API FJsonSchemaMemberPath(const FJsonSchemaMemberPath& InMemberPath);
	FJsonSchemaMemberPath(FJsonSchemaMemberPath&& Other) = default;
	JSONUTILITIES_API explicit FJsonSchemaMemberPath(FStringView InitialString);

	JSONUTILITIES_API FJsonSchemaMemberPath& operator=(const FJsonSchemaMemberPath& Other);
	FJsonSchemaMemberPath& operator=(FJsonSchemaMemberPath&& Other) = default;

	/** Read-only subset of TStringBuilderBase's interface, for compatibility with existing callers. */

	/** Returns the length of the path string. */
	int32 Len() const { return PathString.Len(); }

	/** Returns a pointer to the path string data. */
	const TCHAR* GetData() const { return *PathString; }

	/** Returns a null-terminated TCHAR pointer to the path string. */
	const TCHAR* operator*() const { return *PathString; }

	/** Returns a string view of the path. */
	FStringView ToView() const { return FStringView(PathString); }

	/**
	 * Add a string to the end.
	 * @param String String add to end.
	 * @return Success.
	 */
	JSONUTILITIES_API bool Push(FStringView String);

	/**
	 * Remove a string from the end.
	 * @param String String to remove from end.
	 * @return Success.
	 */
	JSONUTILITIES_API bool Pop(FStringView String);

	/**
	 * Add a string to the end, as a submember: Delimiter + SubmemberText
	 * @param SubmemberString String add to end.
	 * @return Success.
	 */
	JSONUTILITIES_API bool PushSubmember(FStringView SubmemberString);

	/**
	 * Remove a string from the end, as a submember: Delimiter + SubmemberText
	 * @param SubmemberString String to remove from end.
	 * @return Success.
	 */
	JSONUTILITIES_API bool PopSubmember(FStringView SubmemberString);

	/**
	 * Converts path to FText.
	 * @return FText representation of path.
	 */
	FText ToText() const;

private:

	FString PathString;
};

/**
 * Allows RAII scoped control of FJsonSchemaMemberPath. 
 * Pushes string on construction, pops string on destruction.
 * Can be used directly, or with macro below.
 */
struct FJsonSchemaScopedMemberPathPush
{
	UE_NONCOPYABLE(FJsonSchemaScopedMemberPathPush)

	explicit FJsonSchemaScopedMemberPathPush(FJsonSchemaMemberPath& InMemberPath, FStringView InString, const bool bInIsSubmember) : 
		MemberPath(InMemberPath),
		String(InString),
		bIsSubmember(bInIsSubmember)
	{
		if (bIsSubmember)
		{
			MemberPath.PushSubmember(String);
		}
		else
		{
			MemberPath.Push(String);
		}
	}

	~FJsonSchemaScopedMemberPathPush()
	{
		if (bIsSubmember)
		{
			MemberPath.PopSubmember(String);
		}
		else
		{
			MemberPath.Pop(String);
		}
	}

private:
	
	FJsonSchemaMemberPath& MemberPath;
	FString String; // ..can't use FStringView, out of scope when dtr called
	bool bIsSubmember;
};

// Declares uniquely named FScopedSubmemberPath.
#define UE_JSON_SCHEMA_SCOPED_MEMBER_PATH_PUSH(MemberPathExpr, StringExpr, bIsSubmemberExpr) \
	FJsonSchemaScopedMemberPathPush UE_JOIN(_ScopedSubmemberPath_, __LINE__)((MemberPathExpr), (StringExpr), (bIsSubmemberExpr));
