// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "CoreMinimal.h"

#include "JsonObjectConverter.h"

/**
 * Filter used during schema generation, used for both FProperties and UStructs.
 */
struct FJsonSchemaPropertyFilter
{
	/**
	 * Optional callback that will be run when exporting a single property to JsonSchema.
	 * This function should return true if it's handled the property by writing to OutputSchema.	 
	 * If ParameterDefaultString is set then only defaults should be written into the OutputSchema object.
	 */
	using CustomCallback = TDelegate<bool(const FProperty* Property, const FString& ParameterDefaultString, const TSharedRef<FJsonObject>& OutputSchema)>;

	JSONUTILITIES_API explicit FJsonSchemaPropertyFilter(
		const EPropertyFlags InCheckFlags = CPF_None,
		const EPropertyFlags InSkipFlags = CPF_None,
		const TOptional<TSet<FString>>& InRequiredPropertyMemberPaths = TOptional<TSet<FString>>(),
		const TOptional<TSet<FString>>& InSkipPropertyMemberPaths = TOptional<TSet<FString>>(),
		const EJsonObjectConversionFlags InConversionFlags = EJsonObjectConversionFlags::None,
		CustomCallback* CustomCb = nullptr);
	
	JSONUTILITIES_API FJsonSchemaPropertyFilter(const FJsonSchemaPropertyFilter& Other);
	
	/**
	 * If specified (not CPF_None), only properties with any of the specified EPropertyFlags will be considered.
	 */
	EPropertyFlags CheckFlags = CPF_None;

	/**
	 * If specified (not CPF_None), any properties with any of the specified EPropertyFlags will be skipped.
	 */
	EPropertyFlags SkipFlags = CPF_None;

	/**
	 * Only considered while generating schemas for UStructs.
	 * If specified, only properties in this list will be declared in the 'required' schema attribute.
	 * Inner properties should be specified by path e.g: ["StructMember", "StructMember.InnerProperty"]
	 * (If unspecified, all properties without default values in StructMetaData will end up automatically 
	 * listed as required, in code found elsewhere.)
	 */
	TOptional<TSet<FString>> RequiredPropertyMemberPaths;

	/**
	 * Only considered while generating schemas for UStructs.
	 * If specified, any properties in this list will not be described in the resulting schema.
	 * Inner properties should be specified by path e.g: ["StructMember", "StructMember.InnerProperty"]
	 */
	TOptional<TSet<FString>> SkipPropertyMemberPaths;

	/**
	 * If passed, ensures generated schema is compatible with FJsonObjectConverter::UStructToJsonObject executed with the
	 * same flags e.g: EJsonObjectConversionFlags::SkipStandardizeCase for property name casing.
	 */
	EJsonObjectConversionFlags ConversionFlags = EJsonObjectConversionFlags::None;

	/**
	 * If passed, allows for custom property handling.
	 */
	FJsonSchemaPropertyFilter::CustomCallback* CustomCb = nullptr;
	
	/**
	 * Checks if a property should be ignored, to its flags compared to our check/skip flags.
	 * @param Property The property to test.
	 * @return Whether a property should be ignored.
	 */
	JSONUTILITIES_API bool IsPropertyIgnored(TNotNull<const FProperty*> Property) const;

	/**
	 * Checks if a property is required.
	 * @param PropertyMemberPath 
	 * @return Whether the property path is listed as being required.
	 */
	JSONUTILITIES_API bool IsPropertyMemberPathRequired(const FString& PropertyMemberPath) const;

	/**
	 * Checks if a property should be skipped.
	 * @param PropertyMemberPath 
	 * @return Whether the property path is listed as being skipped.
	 */
	JSONUTILITIES_API bool IsPropertyMemberPathSkipped(const FString& PropertyMemberPath) const;

	/**
	 * Converts a property's authored name into a name that respects the ConversionFlags setting.
	 * @param PropertyAuthoredName The authored name of a property.
	 */
	JSONUTILITIES_API FString PropertyAuthoredNameToJsonKey(const FString& PropertyAuthoredName) const;
};
